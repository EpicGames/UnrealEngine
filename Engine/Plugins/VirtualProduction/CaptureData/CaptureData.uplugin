{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Capture Data",
	"Description": "Classes releated to captured data",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Hidden": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "CaptureDataCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureDataEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureDataUtils",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ImgMedia",
			"Enabled": true
		},
		{
			"Name": "CameraCalibrationCore",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled":  true
		}
	]
}