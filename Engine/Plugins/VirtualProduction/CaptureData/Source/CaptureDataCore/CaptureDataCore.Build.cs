// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
public class CaptureDataCore : ModuleRules
{
	public CaptureDataCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Media"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"ImgMedia",
			"CameraCalibrationCore",
			"MeshDescription",
			"StaticMeshDescription",
			"Engine",
			"CaptureDataUtils"
		});
	}
}