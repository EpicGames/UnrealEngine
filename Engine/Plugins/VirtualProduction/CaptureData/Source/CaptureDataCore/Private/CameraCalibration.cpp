// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibration.h"
#include "CaptureDataLog.h"
#include "Models/SphericalLensModel.h"
#include "OpenCVHelperLocal.h"
#include "LensFile.h"
#include "EditorFramework/AssetImportData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/Package.h"
#include "Misc/Paths.h"
#include "VectorTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraCalibration)

void UCameraCalibration::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
}

void UCameraCalibration::PostLoad()
{
	Super::PostLoad();

	// Back-compatability with older import where camera name was not recorded.
	// These always have 2 cameras, the first being rgb the seconds being depth.
	// Distinguish between iphone and HMC import by looking at relative size of RGB and depth images.
	// The RGB camera for the iphone case we'll call "iPhone"
	// The RGB camera for the HMC case will be "bot"

	if (CameraCalibrations.Num() == 2)
	{
		if (CameraCalibrations[0].Name.IsEmpty() && CameraCalibrations[1].Name.IsEmpty())
		{
			if (CameraCalibrations[0].LensFile)
			{
				if (CameraCalibrations[0].LensFile->LensInfo.ImageDimensions.X == CameraCalibrations[1].LensFile->LensInfo.ImageDimensions.X * 2)
				{
					CameraCalibrations[0].Name = TEXT("iPhone");
				}
				else
				{
					CameraCalibrations[0].Name = TEXT("bot");
				}
			}
			else
			{
				CameraCalibrations[0].Name = TEXT("Unknown");
			}

			CameraCalibrations[1].Name = TEXT("Depth");
		}
	}
}

void UCameraCalibration::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
#if WITH_EDITORONLY_DATA
	if (AssetImportData != nullptr)
	{
		OutTags.Emplace(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden);
	}
#endif
}

bool UCameraCalibration::ConvertToTrackerNodeCameraModels(TArray<FCameraCalibration>& OutCalibrations,
														  TArray<TPair<FString, FString>>& OutStereoReconstructionPairs) const
{
	OutCalibrations.SetNum(CameraCalibrations.Num());
	OutStereoReconstructionPairs.SetNum(StereoPairs.Num());

	for (int32 Pair = 0; Pair < OutStereoReconstructionPairs.Num(); Pair++)
	{
		OutStereoReconstructionPairs[Pair] = TPair<FString, FString>(FString::FromInt(StereoPairs[Pair].CameraIndex1), FString::FromInt(StereoPairs[Pair].CameraIndex2));
	}

	for (int32 Cam = 0; Cam < CameraCalibrations.Num(); Cam++)
	{
		FCameraCalibration CurCalib;
		CurCalib.CameraId = CameraCalibrations[Cam].Name;
		CurCalib.CameraType = CameraCalibrations[Cam].IsDepthCamera ? FCameraCalibration::Depth : FCameraCalibration::Video;
		CurCalib.ImageSize.X = CameraCalibrations[Cam].LensFile->LensInfo.ImageDimensions.X;
		CurCalib.ImageSize.Y = CameraCalibrations[Cam].LensFile->LensInfo.ImageDimensions.Y;

		if (CameraCalibrations[Cam].LensFile->LensInfo.LensModel != USphericalLensModel::StaticClass())
		{
			UE_LOGF(LogCaptureDataCore, Warning, "Camera calibration does not contain a SphericalLensModel lens distortion.");
			return false;
		}

		const FDistortionTable& DistortionTable = CameraCalibrations[Cam].LensFile->DistortionTable;
		FDistortionInfo DistortionData;
		bool bGotPoint = DistortionTable.GetPoint(0.0f, 0.0f, DistortionData);

		if (!bGotPoint)
		{
			UE_LOGF(LogCaptureDataCore, Warning, "Camera calibration does not contain a valid lens distortion.");
			return false;
		}

		check(DistortionData.Parameters.Num() == 5);
		CurCalib.K1 = DistortionData.Parameters[0];
		CurCalib.K2 = DistortionData.Parameters[1];
		CurCalib.K3 = DistortionData.Parameters[2]; // parameters are stored K1 K2 K3 P1 P2 rather than OpenCV order of K1 K2 P1 P2 K3
		CurCalib.P1 = DistortionData.Parameters[3];
		CurCalib.P2 = DistortionData.Parameters[4];
		CurCalib.K4 = 0.0;
		CurCalib.K5 = 0.0;
		CurCalib.K6 = 0.0;

		const FFocalLengthTable& FocalLengthTable = CameraCalibrations[Cam].LensFile->FocalLengthTable;
		FFocalLengthInfo FocalLengthData;
		bGotPoint = FocalLengthTable.GetPoint(0.0f, 0.0f, FocalLengthData);

		if (!bGotPoint)
		{
			UE_LOGF(LogCaptureDataCore, Warning, "Camera calibration does not contain a valid focal length.");
			return false;
		}
		const FVector2D& FxFy = FocalLengthData.FxFy;
		CurCalib.FocalLength = FxFy * CurCalib.ImageSize;
		CurCalib.FocalLengthNormalized = FxFy;

		const FImageCenterTable& ImageCenterTable = CameraCalibrations[Cam].LensFile->ImageCenterTable;
		FImageCenterInfo ImageCenterData;
		bGotPoint = ImageCenterTable.GetPoint(0.0f, 0.0f, ImageCenterData);

		if (!bGotPoint)
		{
			UE_LOGF(LogCaptureDataCore, Warning, "Camera calibration does not contain a valid image center.");
			return false;
		}
		const FVector2D& PrincipalPoint = ImageCenterData.PrincipalPoint;
		CurCalib.PrincipalPoint = PrincipalPoint * CurCalib.ImageSize;
		CurCalib.PrincipalPointNormalized = PrincipalPoint;

		const FNodalOffsetTable& NodalOffsetTable = CameraCalibrations[Cam].LensFile->NodalOffsetTable;
		FNodalPointOffset NodalOffsetData;
		bGotPoint = NodalOffsetTable.GetPoint(0.0f, 0.0f, NodalOffsetData);

		if (!bGotPoint)
		{
			UE_LOGF(LogCaptureDataCore, Warning, "Camera calibration does not contain a valid nodal offset.");
			return false;
		}

		FTransform Transform;
		Transform.SetLocation(NodalOffsetData.LocationOffset);
		Transform.SetRotation(NodalOffsetData.RotationOffset);
		FOpenCVHelperLocal::ConvertUnrealToOpenCV(Transform);
		CurCalib.Transform = Transform.ToMatrixWithScale();
		CurCalib.Orientation = CameraOrientation;

		OutCalibrations[Cam] = CurCalib;
	}

	return true;
}

bool UCameraCalibration::ConvertFromTrackerNodeCameraModels(const TArray<FCameraCalibration>& InCalibrations, bool bInUsingUnrealCoordinateSystem)
{
	for (int32 Index = 0; Index < InCalibrations.Num(); ++Index)
	{
		const FCameraCalibration& Calibration = InCalibrations[Index];
		const FString ObjectName = Calibration.CameraType == FCameraCalibration::Depth ?
			FString::Printf(TEXT("%s_Depth_LensFile"), *GetName()) :
			FString::Printf(TEXT("%s_%s_RGB_LensFile"), *GetName(), *Calibration.CameraId);

		CreateLensFileForCalibration(Calibration, ObjectName, bInUsingUnrealCoordinateSystem);
	}

	if (InCalibrations.Num() == 3)
	{
		// stereo HMC so make the stereo pair
		AddStereoPair();
	}

	return true;
}

bool UCameraCalibration::ConvertFromTrackerNodeCameraModels(const TArray<FCameraCalibration>& InCalibrations, const TMap<FString, FString>& InLensAssetNamesMap, bool bInUsingUnrealCoordinateSystem)
{
	for (int32 Index = 0; Index < InCalibrations.Num(); ++Index)
	{
		const FCameraCalibration& Calibration = InCalibrations[Index];

		FString LensFileAssetName;
		if (!InLensAssetNamesMap.IsEmpty() && InLensAssetNamesMap.Find(*Calibration.CameraId))
		{
			LensFileAssetName = InLensAssetNamesMap[*Calibration.CameraId];
		}
		else
		{
			// Fall back to the hardcoded asset names
			LensFileAssetName = Calibration.CameraType == FCameraCalibration::Depth ? 
				FString::Printf(TEXT("%s_Depth_LensFile"), *GetName()) : 
				FString::Printf(TEXT("%s_%s_RGB_LensFile"), *GetName(), *Calibration.CameraId);

			UE_LOGF(LogCaptureDataCore, Warning, "No lens file asset name specified. Using default lens file asset name.");
		}

		CreateLensFileForCalibration(Calibration, LensFileAssetName, bInUsingUnrealCoordinateSystem);
	}

	if (InCalibrations.Num() == 3)
	{
		// stereo HMC so make the stereo pair
		AddStereoPair();
	}

	return true;
}

int32 UCameraCalibration::GetCalibrationIndexByName(const FString& InName) const
{
	for (int32 Index = 0; Index < CameraCalibrations.Num(); ++Index)
	{
		const FExtendedLensFile& Calibration = CameraCalibrations[Index];

		if (Calibration.Name == InName)
		{
			return Index;
		}
	}

	const FString AvailableNames = BuildAvailableCalibrationsString();

	UE_LOGF(
		LogCaptureDataCore,
		Warning,
		"Camera name \"%ls\" could not be found in the camera calibration asset. Available names are %ls",
		*InName,
		*AvailableNames
	);

	return INDEX_NONE;
}

void UCameraCalibration::CreateLensFileForCalibration(const FCameraCalibration& InCalibration, const FString& InAssetName, bool bInUsingUnrealCoordinateSystem)
{
	FString ParentPath = FString::Printf(TEXT("%s/../%s"), *GetPackage()->GetPathName(), *InAssetName);
	FPaths::CollapseRelativeDirectories(ParentPath);
	UObject* Parent = CreatePackage(*ParentPath);
	checkf(Parent, TEXT("Failed to create parent"));

	FExtendedLensFile CameraCalibration;
	CameraCalibration.Name = InCalibration.CameraId;
	CameraCalibration.IsDepthCamera = InCalibration.CameraType == FCameraCalibration::Depth;
	CameraCalibration.LensFile = NewObject<ULensFile>(Parent, ULensFile::StaticClass(), *InAssetName, GetFlags());

	// These a for a non-FIZ camera.
	const float Focus = 0.0f;
	const float Zoom = 0.0f;

	// LensInfo
	CameraCalibration.LensFile->LensInfo.LensModel = USphericalLensModel::StaticClass();
	CameraCalibration.LensFile->LensInfo.LensModelName = FString::Printf(TEXT("Lens"));
	// lens serial number is not needed

	// leave sensor dimensions with default values and de-normalize using VideoDimensions or DepthDimensions
	CameraCalibration.LensFile->LensInfo.ImageDimensions = FIntPoint(InCalibration.ImageSize.X, InCalibration.ImageSize.Y);

	// FocalLengthInfo
	FFocalLengthInfo FocalLengthInfo;
	if (!InCalibration.FocalLengthNormalized.Equals(FVector2D::Zero()))
	{
		FocalLengthInfo.FxFy = InCalibration.FocalLengthNormalized;
	}
	else
	{
		FocalLengthInfo.FxFy = InCalibration.FocalLength / InCalibration.ImageSize;
	}

	// DistortionInfo
	FDistortionInfo DistortionInfo;
	FSphericalDistortionParameters SphericalParameters;

	SphericalParameters.K1 = InCalibration.K1;
	SphericalParameters.K2 = InCalibration.K2;
	SphericalParameters.P1 = InCalibration.P1;
	SphericalParameters.P2 = InCalibration.P2;
	SphericalParameters.K3 = InCalibration.K3;

	USphericalLensModel::StaticClass()->GetDefaultObject<ULensModel>()->ToArray(
		SphericalParameters,
		DistortionInfo.Parameters
	);

	// ImageCenterInfo
	FImageCenterInfo ImageCenterInfo;
	if (!InCalibration.PrincipalPointNormalized.Equals(FVector2D::Zero()))
	{
		ImageCenterInfo.PrincipalPoint = InCalibration.PrincipalPointNormalized;
	}
	else
	{
		ImageCenterInfo.PrincipalPoint = InCalibration.PrincipalPoint / InCalibration.ImageSize;
	}

	// NodalOffset
	FNodalPointOffset NodalPointOffset;
	FTransform Transform;
	Transform.SetFromMatrix(InCalibration.Transform);
	if (!bInUsingUnrealCoordinateSystem)
	{
		FOpenCVHelperLocal::ConvertOpenCVToUnreal(Transform);
	}

	NodalPointOffset.LocationOffset = Transform.GetLocation();
	NodalPointOffset.RotationOffset = Transform.GetRotation();

	if (InCalibration.Orientation == EMediaOrientation::CW90 ||
		InCalibration.Orientation == EMediaOrientation::CW270)
	{
		Swap(CameraCalibration.LensFile->LensInfo.ImageDimensions.X, CameraCalibration.LensFile->LensInfo.ImageDimensions.Y);
		Swap(CameraCalibration.LensFile->LensInfo.SensorDimensions.X, CameraCalibration.LensFile->LensInfo.SensorDimensions.Y);
		Swap(FocalLengthInfo.FxFy.X, FocalLengthInfo.FxFy.Y);

		FVector2D UnrotatedPrincipalPoint = ImageCenterInfo.PrincipalPoint;
		ImageCenterInfo.PrincipalPoint.X = 1.0 - UnrotatedPrincipalPoint.Y;
		ImageCenterInfo.PrincipalPoint.Y = UnrotatedPrincipalPoint.X;
	}

	CameraCalibration.LensFile->AddDistortionPoint(Focus, Zoom, DistortionInfo, FocalLengthInfo);
	CameraCalibration.LensFile->AddImageCenterPoint(Focus, Zoom, ImageCenterInfo);
	CameraCalibration.LensFile->AddNodalOffsetPoint(Focus, Zoom, NodalPointOffset);

	// Add MetaData
	if (UPackage* AssetPackage = CameraCalibration.LensFile->GetPackage())
	{
#if WITH_METADATA
		AssetPackage->GetMetaData().SetValue(CameraCalibration.LensFile, TEXT("CameraId"), *InCalibration.CameraId);
#endif
	}

	CameraCalibration.LensFile->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(CameraCalibration.LensFile);

	CameraCalibrations.Add(CameraCalibration);
	CameraOrientation = InCalibration.Orientation;
}

void UCameraCalibration::AddStereoPair()
{
	FStereoPair CameraPair;
	CameraPair.CameraIndex1 = 0;
	CameraPair.CameraIndex2 = 1;
	StereoPairs.Add(CameraPair);
}

FString UCameraCalibration::BuildAvailableCalibrationsString() const
{
	FString CalibrationNames = TEXT("[");

	if (!CameraCalibrations.IsEmpty())
	{
		CalibrationNames += CameraCalibrations[0].Name;
	}

	for (int32 Index = 1; Index < CameraCalibrations.Num(); ++Index)
	{
		CalibrationNames += FString::Printf(TEXT(", %s"), *CameraCalibrations[Index].Name);
	}

	CalibrationNames += TEXT("]");

	return CalibrationNames;
}
