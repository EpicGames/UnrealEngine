// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureData.h"

#include "CaptureDataLog.h"
#include "CameraCalibration.h"

#include "ImageSequenceUtils.h"

#include "SoundWaveTimecodeUtils.h"

#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "StaticMeshAttributes.h"
#include "MeshAttributes.h"
#include "Engine/StaticMesh.h"
#include "Misc/TransactionObjectEvent.h"
#include "Sound/SoundWave.h"
#include "LensFile.h"
#include "ImgMediaSource.h"
#include "MediaTexture.h"
#include "Algo/AllOf.h"
#include "Engine/SkeletalMesh.h"
#include "ImageSequenceTimecodeUtils.h"

#if WITH_EDITOR
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "CaptureDataEditorBridge.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(CaptureData)

/////////////////////////////////////////////////////
// UCaptureData

#if WITH_EDITOR

void UCaptureData::PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);
	NotifyInternalsChanged();
}

void UCaptureData::PostTransacted(const FTransactionObjectEvent& InTransactionEvent)
{
	Super::PostTransacted(InTransactionEvent);
	NotifyInternalsChanged();
}

#endif

void UCaptureData::NotifyInternalsChanged()
{
	OnCaptureDataInternalsChangedDelegate.Broadcast();
}

/////////////////////////////////////////////////////
// UMeshCaptureData

bool UMeshCaptureData::IsInitialized(ECaptureDataInitializedCheck InInitializedCheck) const
{
	// We need access to the source data when fitting and that is not available during runtime
#if WITH_EDITOR
	return TargetMesh != nullptr && (TargetMesh->IsA<UStaticMesh>() || TargetMesh->IsA<USkeletalMesh>());
#else
	return false;
#endif
}

void UMeshCaptureData::GetDataForConforming(const FTransform& InTransform, TArray<float>& OutVertices, TArray<int32>& OutTriangles) const
{
#if WITH_EDITOR
	if (USkeletalMesh* TargetSkeletalMesh = Cast<USkeletalMesh>(TargetMesh))
	{

		FSkeletalMeshModel* ImportedModel = TargetSkeletalMesh->GetImportedModel();
		FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[0];

		OutVertices.Reset(LODModel.NumVertices * 3);

		for (FSkelMeshSection& Section : LODModel.Sections)
		{
			const int32 NumVertices = Section.GetNumVertices();
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				FSoftSkinVertex& OriginalVertex = Section.SoftVertices[VertexIndex];
				FVector TransformedVertex = InTransform.TransformPosition(FVector{ OriginalVertex.Position });
				OutVertices.Add(TransformedVertex.Y);
				OutVertices.Add(-TransformedVertex.Z);
				OutVertices.Add(TransformedVertex.X);
			}
		}

		OutTriangles.Reset(LODModel.IndexBuffer.Num());
		for (uint32 Index : LODModel.IndexBuffer)
		{
			OutTriangles.Add(Index);
		}
	}
	else if (UStaticMesh* TargetStaticMesh = Cast<UStaticMesh>(TargetMesh))
	{
		FMeshDescription* MeshDescription = TargetStaticMesh->GetMeshDescription(0);
		check(MeshDescription);
		FStaticMeshAttributes Attributes(*MeshDescription);

		TVertexAttributesRef<FVector3f> OriginalMeshVerts = Attributes.GetVertexPositions();
		TTriangleAttributesRef<TArrayView<FVertexID>>  OriginalMeshIndices = Attributes.GetTriangleVertexIndices();

		OutVertices.Reset(OriginalMeshVerts.GetNumElements());

		for (int32 RenderCtr = 0; RenderCtr < OriginalMeshVerts.GetNumElements(); ++RenderCtr)
		{
			// map the mesh vertices (in UE coordinate system) to OpenCV coordinate system
			FVector3f OriginalVertex = OriginalMeshVerts.Get(RenderCtr);
			FVector TransformedVertex = InTransform.TransformPosition(FVector{ OriginalVertex });
			OutVertices.Add(TransformedVertex.Y);
			OutVertices.Add(-TransformedVertex.Z);
			OutVertices.Add(TransformedVertex.X);
		}

		OutTriangles.Reset(OriginalMeshIndices.GetNumElements());
		const TArrayView<FVertexID>& RawIndArray = OriginalMeshIndices.GetRawArray();
		for (const auto& Index : RawIndArray)
		{
			OutTriangles.Add(Index.GetValue());
		}
	}
	else
	{
		// This is an error state so log it accordingly
		if (TargetMesh != nullptr)
		{
			UE_LOGF(LogCaptureDataCore, Error, "Failed to get data for conforming as TargetMesh is a '%ls' but should be a UStaticMesh or USkeletalMesh", *TargetMesh->GetClass()->GetName());
		}
		else
		{
			UE_LOGF(LogCaptureDataCore, Error, "Failed to get data for conforming as TargetMesh is invalid");
		}
	}
#endif
}

/////////////////////////////////////////////////////
// UFootageCaptureData

UFootageCaptureData::FPathAssociation::FPathAssociation(const FString& InPathOnDisk, const FString& InAssetPath) :
	PathOnDisk(InPathOnDisk),
	AssetPath(InAssetPath)
{
}

TOptional<FFootageCaptureMetadata::FIosDeviceVersion> FFootageCaptureMetadata::ParseIosDeviceVersion(const FString& Prefix, const FString& ModelName)
{
	FString CombinedModelNumbers;
	if (ModelName.Split(Prefix, nullptr, &CombinedModelNumbers)) // Combined model numbers, e.g. "1,2"
	{
		TArray<FString> SeparatedModelNumbers;
		if (CombinedModelNumbers.ParseIntoArray(SeparatedModelNumbers, TEXT(",")) == 2) // Separated model numbers, e.g. ["1", "2"]
		{
			return FIosDeviceVersion(FCString::Atoi(*SeparatedModelNumbers[0]), FCString::Atoi(*SeparatedModelNumbers[1]));
		}
	}

	return TOptional<FIosDeviceVersion>();
}

EFootageDeviceClass FFootageCaptureMetadata::IPhoneDeviceClass(TOptional<FIosDeviceVersion> IosDeviceVersion)
{
	if (IosDeviceVersion.IsSet())
	{
		const FIosDeviceVersion Version = IosDeviceVersion.GetValue();
		const uint16 Major = Version.Major;
		const uint16 Minor = Version.Minor;
		if (Major < 12) // Before iPhone 11
		{
			return EFootageDeviceClass::iPhone11OrEarlier;
		}
		else if (Major == 12) // iPhone 11
		{
			const TArray<uint16> IPhone11MinorVersions = { 1, 3, 5 };
			if (IPhone11MinorVersions.Contains(Minor))
			{
				return EFootageDeviceClass::iPhone11OrEarlier;
			}
		}
		else if (Major == 13) // iPhone 12
		{
			return EFootageDeviceClass::iPhone12;
		}
		else if (Major == 14) // iPhone 13 or non pro iPhone 14 models
		{
			const TArray<uint16> IPhone13MinorVersions = { 2, 3, 4, 5 };
			const TArray<uint16> IPhone14MinorVersions = { 7, 8 };
			if (IPhone13MinorVersions.Contains(Minor))
			{
				return EFootageDeviceClass::iPhone13;
			}
			else if (IPhone14MinorVersions.Contains(Minor))
			{
				return EFootageDeviceClass::iPhone14OrLater;
			}
		}
		else if (Major >= 15) // iPhone 14 Pro Models or later
		{
			return  EFootageDeviceClass::iPhone14OrLater;
		}
	}

	return EFootageDeviceClass::OtheriOSDevice;
}

void FFootageCaptureMetadata::SetDeviceClass(const FString& InDeviceModel)
{
	const FString IPhone = TEXT("iPhone");
	const FString IPad = TEXT("iPad");
	const FString StereoHMC = TEXT("StereoHMC");

	if (InDeviceModel.StartsWith(IPhone))
	{
		DeviceClass = IPhoneDeviceClass(ParseIosDeviceVersion(IPhone, InDeviceModel));
	}
	else if (InDeviceModel.StartsWith(IPad))
	{
		DeviceClass = EFootageDeviceClass::OtheriOSDevice;
	}
	else if (InDeviceModel == StereoHMC)
	{
		DeviceClass = EFootageDeviceClass::StereoHMC;
	}
	else
	{
		DeviceClass = EFootageDeviceClass::Unspecified;
	}
}

#define CHECK_AND_PRINT_RET(InCondition, InFormat, ...) if (!(InCondition)) { return MakeError(FString::Printf(InFormat, ##__VA_ARGS__)); }
#define CHECK_AND_RET_ERROR(InFunction) if (UFootageCaptureData::FVerifyResult Result = InFunction; Result.HasError()) { return MoveTemp(Result); }

bool UFootageCaptureData::IsInitialized(ECaptureDataInitializedCheck InInitializedCheck) const
{
	return !VerifyData(InInitializedCheck).HasError();
}

UFootageCaptureData::FVerifyResult UFootageCaptureData::VerifyData(ECaptureDataInitializedCheck InInitializedCheck) const
{
	CHECK_AND_RET_ERROR(ViewsContainsValidData(InInitializedCheck));

	if (EnumHasAnyFlags(InInitializedCheck, ECaptureDataInitializedCheck::Calibrations))
	{
		CHECK_AND_RET_ERROR(CalibrationContainsValidData());
	}

	if (EnumHasAnyFlags(InInitializedCheck, ECaptureDataInitializedCheck::Metadata))
	{
		CHECK_AND_RET_ERROR(MetadataContainsValidData());
	}

	return MakeValue();
}

TArray<UFootageCaptureData::FPathAssociation> UFootageCaptureData::CheckImageSequencePaths() const
{
	TArray<FPathAssociation> InvalidImageSequences;

	for (const TObjectPtr<class UImgMediaSource>& ImageSequence : ImageSequences)
	{
		if (ImageSequence && !FPaths::DirectoryExists(ImageSequence->GetFullPath()))
		{
			InvalidImageSequences.Emplace(ImageSequence->GetFullPath(), ImageSequence->GetPathName());
		}
	}

	return InvalidImageSequences;
}

UFootageCaptureData::FVerifyResult UFootageCaptureData::ViewsContainsValidData(ECaptureDataInitializedCheck InInitializedCheck) const
{
	auto IsFrameRateValid = [](const FFrameRate& InFrameRate)
		{
			return InFrameRate.Numerator > 0 && InFrameRate.Denominator > 0;
		};

	if (EnumHasAnyFlags(InInitializedCheck, ECaptureDataInitializedCheck::ImageSequences))
	{
		CHECK_AND_PRINT_RET(!ImageSequences.IsEmpty(), TEXT("Capture Data doesn't contain image sequences"));

		for (int32 Index = 0; Index < ImageSequences.Num(); ++Index)
		{
			UImgMediaSource* ImageSequence = ImageSequences[Index];

			CHECK_AND_PRINT_RET(FImageSequenceUtils::IsImageSequenceValid(ImageSequence), TEXT("Image Sequence asset is invalid"));
			CHECK_AND_PRINT_RET(IsFrameRateValid(ImageSequence->FrameRateOverride), TEXT("Image Sequence asset contains invalid frame rate"));
		}
	}

	if (EnumHasAnyFlags(InInitializedCheck, ECaptureDataInitializedCheck::DepthSequences))
	{
		CHECK_AND_PRINT_RET(!DepthSequences.IsEmpty(), TEXT("Capture Data doesn't contain depth sequences"));

		for (int32 Index = 0; Index < DepthSequences.Num(); ++Index)
		{
			UImgMediaSource* DepthSequence = DepthSequences[Index];

			CHECK_AND_PRINT_RET(FImageSequenceUtils::IsImageSequenceValid(DepthSequence), TEXT("Depth Sequence asset is invalid"));
			CHECK_AND_PRINT_RET(IsFrameRateValid(DepthSequence->FrameRateOverride), TEXT("Depth Sequence asset contains invalid frame rate"));
		}
	}

	return MakeValue();
}

UFootageCaptureData::FVerifyResult UFootageCaptureData::MetadataContainsValidData() const
{
	CHECK_AND_PRINT_RET(!FMath::IsNearlyZero(Metadata.FrameRate), TEXT("Frame rate can't be set to 0"));

	return MakeValue();
}

UFootageCaptureData::FVerifyResult UFootageCaptureData::CalibrationContainsValidData() const
{
	CHECK_AND_PRINT_RET(!CameraCalibrations.IsEmpty(), TEXT("Calibration assets are empty"));
	for (UCameraCalibration* CameraCalibration : CameraCalibrations)
	{
		CHECK_AND_PRINT_RET(CameraCalibration != nullptr, TEXT("Calibration asset not configured"));
		CHECK_AND_PRINT_RET(!CameraCalibration->CameraCalibrations.IsEmpty(), TEXT("Calibration asset not configured"));
	}
	return MakeValue();
}

#undef CHECK_AND_PRINT_RET
#undef CHECK_AND_RET

static TRange<FFrameNumber> GetFrameRange(const FFrameRate& InTargetRate, const FTimecode& InMediaTimecode, const FFrameRate& InMediaTimecodeRate, bool bInMediaStartFrameIsZero, const FFrameRate& InMediaRate, const FFrameNumber& InMediaDuration)
{
	// FFrameRate::TransformTime crashes on a zero-numerator source rate, and rates can still be
	// uninitialized while an asset is being edited in the editor. Returning an empty range avoids
	// feeding a bad rate into the transform.
	auto IsFrameRateUsable = [](const FFrameRate& InRate)
	{
		return InRate.Numerator > 0 && InRate.Denominator > 0;
	};

	if (!IsFrameRateUsable(InTargetRate) || !IsFrameRateUsable(InMediaTimecodeRate) || !IsFrameRateUsable(InMediaRate))
	{
		UE_LOGF(LogCaptureDataCore, Warning,
			"GetFrameRange skipped: invalid frame rate (Target=%d/%d, MediaTimecode=%d/%d, Media=%d/%d). Returning empty range.",
			InTargetRate.Numerator, InTargetRate.Denominator,
			InMediaTimecodeRate.Numerator, InMediaTimecodeRate.Denominator,
			InMediaRate.Numerator, InMediaRate.Denominator);
		return TRange<FFrameNumber>(0, 0);
	}

	const FFrameNumber MediaStartFrame = bInMediaStartFrameIsZero ? 0 : InMediaTimecode.ToFrameNumber(InMediaTimecodeRate);
	const FFrameTime TargetStartFrameTime = FFrameRate::TransformTime(MediaStartFrame, InMediaTimecodeRate, InTargetRate);
	const FFrameTime TargetDurationTime = FFrameRate::TransformTime(InMediaDuration, InMediaRate, InTargetRate);
	const FFrameTime TargetEndFrameTime = TargetStartFrameTime + TargetDurationTime;
	// When converting from frame time to frame number, deal with sub frame times by taking minimum frame range.
	TRange<FFrameNumber> TargetFrameRange(TargetStartFrameTime.CeilToFrame(), TargetEndFrameTime.FloorToFrame());
	return TargetFrameRange;
}

static TRange<FFrameNumber> GetFrameRange(const FFrameRate& InTargetRate, UImgMediaSource* InMedia, const FTimecode& InMediaTimecode, const FFrameRate& InMediaTimecodeRate, bool bInMediaStartFrameIsZero)
{
	int32 Duration = 0;
	FIntVector2 ImageDimensions;
	const bool bImageOK = FImageSequenceUtils::GetImageSequenceInfoFromAsset(InMedia, ImageDimensions, Duration);
	ensure(bImageOK);

	return GetFrameRange(InTargetRate, InMediaTimecode, InMediaTimecodeRate, bInMediaStartFrameIsZero, InMedia->FrameRateOverride, Duration);
}

static TRange<FFrameNumber> GetFrameRange(const FFrameRate& InTargetRate, USoundWave* InMedia, const FTimecode& InMediaTimecode, const FFrameRate& InMediaTimecodeRate, bool bInMediaStartFrameIsZero)
{
	const int32 Duration = InMedia->GetDuration() * InMediaTimecodeRate.AsDecimal();

	return GetFrameRange(InTargetRate, InMediaTimecode, InMediaTimecodeRate, bInMediaStartFrameIsZero, InMediaTimecodeRate, Duration);
}

FIntPoint UFootageCaptureData::GetFootageColorResolution() const
{
	FIntPoint Resolution(ForceInit);

	if (!CameraCalibrations.IsEmpty())
	{
		TArray<FCameraCalibration> Calibrations;
		TArray<TPair<FString, FString>> StereoPairs;
		CameraCalibrations[0]->ConvertToTrackerNodeCameraModels(Calibrations, StereoPairs);

		FCameraCalibration* VideoCalibration = Calibrations.FindByPredicate([](const FCameraCalibration& InCalibration)
		{
			return InCalibration.CameraType == FCameraCalibration::Video;
		});

		if (VideoCalibration)
		{
			Resolution = FIntPoint(VideoCalibration->ImageSize.X, VideoCalibration->ImageSize.Y);
		}
	}

	return Resolution;
}

void UFootageCaptureData::GetFrameRanges(const FFrameRate& InTargetRate, ETimecodeAlignment InTimecodeAlignment, bool bInIncludeAudio, TMap<TWeakObjectPtr<UObject>, TRange<FFrameNumber>>& OutMediaFrameRanges, TRange<FFrameNumber>& OutProcessingFrameRange, TRange<FFrameNumber>& OutMaximumFrameRange) const
{
	OutMediaFrameRanges.Reset();
	OutProcessingFrameRange = TRange<FFrameNumber>(0, 0);
	OutMaximumFrameRange = TRange<FFrameNumber>(0, 0);

	const bool bMediaStartFrameIsZero = InTimecodeAlignment == ETimecodeAlignment::None;

	for (int32 Index = 0; Index < ImageSequences.Num(); ++Index)
	{
		const TObjectPtr<UImgMediaSource>& ImageSequence = ImageSequences[Index];

		if (ImageSequence)
		{
			const FTimecode EffectiveImageTimecode = GetEffectiveImageTimecode(Index);
			const FFrameRate EffectiveImageTimecodeRate = GetEffectiveImageTimecodeRate(Index);
			const TRange<FFrameNumber> ImageFrameFrange = GetFrameRange(InTargetRate, ImageSequence, EffectiveImageTimecode, EffectiveImageTimecodeRate, bMediaStartFrameIsZero);

			OutMediaFrameRanges.Add(ImageSequence, ImageFrameFrange);
		}
	}

	for (int32 Index = 0; Index < DepthSequences.Num(); ++Index)
	{
		const TObjectPtr<UImgMediaSource>& DepthSequence = DepthSequences[Index];

		if (DepthSequence)
		{
			const FTimecode EffectiveDepthTimecode = GetEffectiveDepthTimecode(Index);
			const FFrameRate EffectiveDepthTimecodeRate = GetEffectiveDepthTimecodeRate(Index);
			const TRange<FFrameNumber> DepthFrameRange = GetFrameRange(InTargetRate, DepthSequence, EffectiveDepthTimecode, EffectiveDepthTimecodeRate, bMediaStartFrameIsZero);

			OutMediaFrameRanges.Add(DepthSequence, DepthFrameRange);
		}
	}

	if (bInIncludeAudio)
	{
		for (TObjectPtr<class USoundWave> Audio : AudioTracks)
		{
			if (Audio)
			{
				const FTimecode EffectiveAudioTimecode = GetEffectiveAudioTimecode();
				const FFrameRate EffectiveAudioTimecodeRate = GetEffectiveAudioTimecodeRate();
				const TRange<FFrameNumber> AudioFrameRange = GetFrameRange(InTargetRate, Audio, EffectiveAudioTimecode, EffectiveAudioTimecodeRate, bMediaStartFrameIsZero);

				OutMediaFrameRanges.Add(Audio, AudioFrameRange);
			}
		}
	}

	if (InTimecodeAlignment == ETimecodeAlignment::Relative)
	{
		FFrameNumber LowestStartFrame;

		bool bFirstPass = true;
		for (const TPair<TWeakObjectPtr<UObject>, TRange<FFrameNumber>>& MediaFrameRangePair : OutMediaFrameRanges)
		{
			if (bFirstPass || MediaFrameRangePair.Value.GetLowerBoundValue() < LowestStartFrame)
			{
				LowestStartFrame = MediaFrameRangePair.Value.GetLowerBoundValue();
			}

			bFirstPass = false;
		}

		for (TPair<TWeakObjectPtr<UObject>, TRange<FFrameNumber>>& MediaFrameRangePair : OutMediaFrameRanges)
		{
			MediaFrameRangePair.Value.SetLowerBoundValue(MediaFrameRangePair.Value.GetLowerBoundValue() - LowestStartFrame);
			MediaFrameRangePair.Value.SetUpperBoundValue(MediaFrameRangePair.Value.GetUpperBoundValue() - LowestStartFrame);
		}
	}

	bool bFirstPass = true;
	for (const TPair<TWeakObjectPtr<UObject>, TRange<FFrameNumber>>& MediaFrameRangePair : OutMediaFrameRanges)
	{
		if (MediaFrameRangePair.Key.IsValid() &&
			(MediaFrameRangePair.Key->IsA(UImgMediaSource::StaticClass()) || MediaFrameRangePair.Key->IsA(USoundWave::StaticClass())))
		{
			if (bFirstPass || MediaFrameRangePair.Value.GetLowerBoundValue() > OutProcessingFrameRange.GetLowerBoundValue())
			{
				OutProcessingFrameRange.SetLowerBoundValue(MediaFrameRangePair.Value.GetLowerBoundValue());
			}

			if (bFirstPass || MediaFrameRangePair.Value.GetUpperBoundValue() < OutProcessingFrameRange.GetUpperBoundValue())
			{
				OutProcessingFrameRange.SetUpperBoundValue(MediaFrameRangePair.Value.GetUpperBoundValue());
			}

			if (bFirstPass || MediaFrameRangePair.Value.GetLowerBoundValue() < OutMaximumFrameRange.GetLowerBoundValue())
			{
				OutMaximumFrameRange.SetLowerBoundValue(MediaFrameRangePair.Value.GetLowerBoundValue());
			}

			if (bFirstPass || MediaFrameRangePair.Value.GetUpperBoundValue() > OutMaximumFrameRange.GetUpperBoundValue())
			{
				OutMaximumFrameRange.SetUpperBoundValue(MediaFrameRangePair.Value.GetUpperBoundValue());
			}

			bFirstPass = false;
		}
	}

	if (OutProcessingFrameRange.GetUpperBoundValue() <= OutProcessingFrameRange.GetLowerBoundValue())
	{
		OutProcessingFrameRange = TRange<FFrameNumber>(0, 0);
	}
}

TRange<FFrameNumber> UFootageCaptureData::GetAudioFrameRange(const FFrameRate& InTargetRate, ETimecodeAlignment InTimecodeAlignment, USoundWave* InMedia, const FTimecode& InMediaTimecode, const FFrameRate& InMediaTimecodeRate)
{
	bool bInMediaStartFrameIsZero = InTimecodeAlignment == ETimecodeAlignment::None;
	TRange<FFrameNumber> AudioFrameRange = GetFrameRange(InTargetRate, InMedia, InMediaTimecode, InMediaTimecodeRate, bInMediaStartFrameIsZero);

	if (InTimecodeAlignment == ETimecodeAlignment::Relative)
	{
		FFrameNumber StartFrame = AudioFrameRange.GetLowerBoundValue();
		AudioFrameRange.SetLowerBoundValue(AudioFrameRange.GetLowerBoundValue() - StartFrame);
		AudioFrameRange.SetUpperBoundValue(AudioFrameRange.GetUpperBoundValue() - StartFrame);
	}

	return AudioFrameRange;
}

FTimecode UFootageCaptureData::GetEffectiveImageTimecode(int32 InView) const
{
	check(InView < ImageSequences.Num());

	FTimecode ImgSequenceTimecode = UImageSequenceTimecodeUtils::GetTimecode(ImageSequences[InView]);
	if (UImageSequenceTimecodeUtils::IsValidTimecode(ImgSequenceTimecode))
	{
		return ImgSequenceTimecode;
	}

	FTimecode Timecode;
	FFrameRate FrameRate;
	GetDefaultTimecodeInfo(Timecode, FrameRate);

	return Timecode;
}

FFrameRate UFootageCaptureData::GetEffectiveImageTimecodeRate(int32 InView) const
{
	check(InView < ImageSequences.Num());

	FFrameRate ImgSequenceFrameRate = UImageSequenceTimecodeUtils::GetFrameRate(ImageSequences[InView]);
	if (UImageSequenceTimecodeUtils::IsValidFrameRate(ImgSequenceFrameRate))
	{
		return ImgSequenceFrameRate;
	}

	FTimecode Timecode;
	FFrameRate FrameRate;
	GetDefaultTimecodeInfo(Timecode, FrameRate);

	return FrameRate;
}

FTimecode UFootageCaptureData::GetEffectiveDepthTimecode(int32 InView) const
{
	check(InView < DepthSequences.Num());

	FTimecode DepthSequenceTimecode = UImageSequenceTimecodeUtils::GetTimecode(DepthSequences[InView]);
	if (UImageSequenceTimecodeUtils::IsValidTimecode(DepthSequenceTimecode))
	{
		return DepthSequenceTimecode;
	}

	FTimecode Timecode;
	FFrameRate FrameRate;
	GetDefaultTimecodeInfo(Timecode, FrameRate);

	return Timecode;
}

FFrameRate UFootageCaptureData::GetEffectiveDepthTimecodeRate(int32 InView) const
{
	check(InView < DepthSequences.Num());

	FFrameRate DepthSequenceFrameRate = UImageSequenceTimecodeUtils::GetFrameRate(DepthSequences[InView]);
	if (UImageSequenceTimecodeUtils::IsValidFrameRate(DepthSequenceFrameRate))
	{
		return DepthSequenceFrameRate;
	}
	
	FTimecode Timecode;
	FFrameRate FrameRate;
	GetDefaultTimecodeInfo(Timecode, FrameRate);

	return FrameRate;
}

FTimecode UFootageCaptureData::GetEffectiveAudioTimecode() const
{
	FTimecode Timecode;
	bool TimecodeFromAsset = false;

	if (AudioTracks.Num() > 0)
	{
		const USoundWave* SoundWave = AudioTracks[0].Get();
		if (SoundWave != nullptr)
		{
			// Checks the timecode rate as well
			if (USoundWaveTimecodeUtils::HasTimecode(SoundWave))
			{
				Timecode = USoundWaveTimecodeUtils::GetTimecode(SoundWave);
				TimecodeFromAsset = true;
			}
		}
	}

	if (!TimecodeFromAsset)
	{
		FFrameRate FrameRate;
		GetDefaultTimecodeInfo(Timecode, FrameRate);
	}

	return Timecode;
}

FFrameRate UFootageCaptureData::GetEffectiveAudioTimecodeRate() const
{
	FFrameRate FrameRate;
	bool TimecodeRateFromAsset = false;

	if (AudioTracks.Num() > 0)
	{
		const USoundWave* SoundWave = AudioTracks[0].Get();
		if (SoundWave != nullptr)
		{
			if (USoundWaveTimecodeUtils::HasTimecodeRate(SoundWave))
			{
				FrameRate = USoundWaveTimecodeUtils::GetFrameRate(SoundWave);
				TimecodeRateFromAsset = true;
			}
		}
	}

	if (!TimecodeRateFromAsset)
	{
		FTimecode Timecode;
		GetDefaultTimecodeInfo(Timecode, FrameRate);
	}

	return FrameRate;
}

void UFootageCaptureData::GetDefaultTimecodeInfo(FTimecode& OutTimecode, FFrameRate& OutFrameRate) const
{
	bool bTimecodeSet = false;
	bool bFrameRateSet = false;

	for (const TObjectPtr<class UImgMediaSource>& ImageSequence : ImageSequences)
	{
		if (ImageSequence.Get() != nullptr)
		{
			FTimecode Timecode = UImageSequenceTimecodeUtils::GetTimecode(ImageSequence);
			FFrameRate FrameRate = UImageSequenceTimecodeUtils::GetFrameRate(ImageSequence);

			if (!bTimecodeSet && UImageSequenceTimecodeUtils::IsValidTimecode(Timecode))
			{
				OutTimecode = MoveTemp(Timecode);
				bTimecodeSet = true;
			}

			if (!bFrameRateSet && UImageSequenceTimecodeUtils::IsValidFrameRate(FrameRate))
			{
				OutFrameRate = MoveTemp(FrameRate);
				bFrameRateSet = true;
			}
		}
	}

	for (const TObjectPtr<class UImgMediaSource>& DepthSequence : DepthSequences)
	{
		if (DepthSequence.Get() != nullptr)
		{
			FTimecode Timecode = UImageSequenceTimecodeUtils::GetTimecode(DepthSequence);
			FFrameRate FrameRate = UImageSequenceTimecodeUtils::GetFrameRate(DepthSequence);

			if (!bTimecodeSet && UImageSequenceTimecodeUtils::IsValidTimecode(Timecode))
			{
				OutTimecode = MoveTemp(Timecode);
				bTimecodeSet = true;
			}

			if (!bFrameRateSet && UImageSequenceTimecodeUtils::IsValidFrameRate(FrameRate))
			{
				OutFrameRate = MoveTemp(FrameRate);
				bFrameRateSet = true;
			}
		}
	}

	if (AudioTracks.Num() > 0)
	{
		const USoundWave* SoundWave = AudioTracks[0].Get();
		if (SoundWave != nullptr) 
		{
			// Checks the timecode rate as well
			if (!bTimecodeSet && USoundWaveTimecodeUtils::HasTimecode(SoundWave))
			{
				OutTimecode = USoundWaveTimecodeUtils::GetTimecode(SoundWave);
				bTimecodeSet = true;
			}
			
			if (!bFrameRateSet)
			{
				OutFrameRate = USoundWaveTimecodeUtils::GetFrameRate(SoundWave);
				bFrameRateSet = true;
			}
		}
	}

	if (!bTimecodeSet)
	{
		OutTimecode = FTimecode(0, 0, 0, 0, false);
	}

	if (!bFrameRateSet)
	{
		OutFrameRate = FFrameRate(30, 1);
	}
}

void UFootageCaptureData::PostLoad()
{
	Super::PostLoad();

	if (Metadata.DeviceModel_DEPRECATED != EFootageDeviceClass::Unspecified)
	{
		Metadata.DeviceClass = Metadata.DeviceModel_DEPRECATED;
		Metadata.DeviceModel_DEPRECATED = EFootageDeviceClass::Unspecified;
	}

	if (CameraCalibration_DEPRECATED != nullptr)
	{
		CameraCalibrations.Add(MoveTemp(CameraCalibration_DEPRECATED));
		CameraCalibration_DEPRECATED = nullptr;
	}

	if (Audio_DEPRECATED != nullptr)
	{
		AudioTracks.Add(MoveTemp(Audio_DEPRECATED));
		Audio_DEPRECATED = nullptr;
	}

	if (!Audios_DEPRECATED.IsEmpty())
	{
		for (TObjectPtr<USoundWave> Audio : Audios_DEPRECATED)
		{
			AudioTracks.Add(MoveTemp(Audio));
		}
		Audios_DEPRECATED.Empty();
	}

	TQueue<TWeakObjectPtr<UObject>> ObjectsToMarkDirty;

#if WITH_EDITOR
	for (FFootageCaptureView& View : Views_DEPRECATED)
	{
		if (View.bImageTimecodePresent)
		{
			UImageSequenceTimecodeUtils::SetTimecodeInfo(View.ImageTimecode, View.ImageTimecodeRate, View.ImageSequence.Get());
		}

		ObjectsToMarkDirty.Enqueue(View.ImageSequence);
		ImageSequences.Add(MoveTemp(View.ImageSequence));

		if (View.bDepthTimecodePresent)
		{
			UImageSequenceTimecodeUtils::SetTimecodeInfo(View.DepthTimecode, View.DepthTimecodeRate, View.DepthSequence.Get());
		}

		ObjectsToMarkDirty.Enqueue(View.DepthSequence);
		DepthSequences.Add(MoveTemp(View.DepthSequence));
	}
#endif
	if (AudioTracks.Num() > 0 && bAudioTimecodePresent_DEPRECATED == true)
	{
		USoundWave* SoundWave = AudioTracks[0].Get();
		check(IsValid(SoundWave));

		USoundWaveTimecodeUtils::SetTimecodeInfo(AudioTimecode_DEPRECATED, AudioTimecodeRate_DEPRECATED, SoundWave);
		bAudioTimecodePresent_DEPRECATED = false;
		ObjectsToMarkDirty.Enqueue(SoundWave);
	}

	// Add the CameraId metadata tag to LensFiles if not already set
	for (TObjectPtr<UCameraCalibration> CalibrationAsset : CameraCalibrations)
	{
		if (IsValid(CalibrationAsset))
		{
			for (FExtendedLensFile& Calibration : CalibrationAsset->CameraCalibrations)
			{
				ULensFile* LensFile = Calibration.LensFile;
				if (IsValid(LensFile))
				{
					if (UPackage* AssetPackage = LensFile->GetPackage())
					{
#if WITH_METADATA
						if (AssetPackage && !AssetPackage->GetMetaData().HasValue(LensFile, TEXT("CameraId")))
						{
							AssetPackage->GetMetaData().SetValue(LensFile, TEXT("CameraId"), *Calibration.Name);
							ObjectsToMarkDirty.Enqueue(LensFile);
						}
#endif
					}
				}
			}
		}
	}

#if WITH_EDITOR
	// The calibration and capture data assets get marked dirty by the FCaptureDataCoreModule::CheckAssetMigration()
	// function, so we need to mark the other assets we've modified here as dirty as well. Otherwise if a user closes
	// the editor after this point they will be prompted to save only the calibration and capture data assets and not
	// the image sequences, depth sequences etc., if that happens the timecode information we've migrated here will get
	// lost, as those changes reside only in memory and the capture data PostLoad switches to update them won't get
	// triggered again once the capture data asset is saved.
	//
	// We defer the mark until after PostLoad, to a point when the editor and relevant subsystems are known to be ready.

	if (!ObjectsToMarkDirty.IsEmpty())
	{
		using namespace UE::CaptureManager;

		ICaptureDataEditorBridge& EditorBridge = FModuleManager::LoadModuleChecked<ICaptureDataEditorBridge>("CaptureDataEditor");

		TWeakObjectPtr<UObject> ObjectToMarkDirty;

		while (ObjectsToMarkDirty.Dequeue(ObjectToMarkDirty))
		{
			EditorBridge.DeferMarkDirty(MoveTemp(ObjectToMarkDirty));
		}
	}
#endif
}

void UFootageCaptureData::PopulateCameraNames(UFootageCaptureData* InFootageCaptureData, FString& InOutCamera, TArray<TSharedPtr<FString>>& OutCameraNames)
{
	OutCameraNames.Reset();

	if (InFootageCaptureData)
	{
		if (InFootageCaptureData->CameraCalibrations.IsEmpty())
		{
			for (int32 Index = 0; Index < InFootageCaptureData->ImageSequences.Num(); ++Index)
			{
				OutCameraNames.Add(MakeShared<FString>(FString::Printf(TEXT("Camera %i"), Index))); // If you change the format also update UFootageCaptureData::GetViewIndexByCameraName
			}
		}
		else
		{
			const UCameraCalibration* CameraCalibration = InFootageCaptureData->CameraCalibrations[0];
			if (CameraCalibration)
			{
				for (const FExtendedLensFile& LensFile : CameraCalibration->CameraCalibrations)
				{
					if (!LensFile.IsDepthCamera)
					{
						OutCameraNames.Add(MakeShared<FString>(LensFile.Name));
					}
				}
			}
		}
	}

	if (OutCameraNames.IsEmpty())
	{
		InOutCamera = TEXT("");
	}
	else
	{
		bool bNameFound = false;

		for (TSharedPtr<FString> Name : OutCameraNames)
		{
			if (*Name == InOutCamera)
			{
				bNameFound = true;
				break;
			}
		}

		if (!bNameFound)
		{
			InOutCamera = *OutCameraNames[0];
		}
	}
}

#if WITH_EDITORONLY_DATA
static int32 FindCameraByName(const TArray<TObjectPtr<UImgMediaSource>>& InSequence, const FString& InName)
{
	for (int32 Index = 0; Index < InSequence.Num(); ++Index)
	{
		TObjectPtr<const UImgMediaSource> ImageSequence = InSequence[Index];

		UPackage* Package = ImageSequence.GetPackage();
		check(Package);

		FMetaData& MetaData = Package->GetMetaData();
		if (!MetaData.HasValue(ImageSequence, "CameraId"))
		{
			// End the loop early as we are not supporting having partial metadata
			return INDEX_NONE;
		}

		FString CameraId = MetaData.GetValue(ImageSequence, "CameraId");
		if (CameraId == InName)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}
#endif

static int32 CheckCalibrationArray(const TArray<FExtendedLensFile>& InCalibrationArray, const FString& InName, bool bIsDepth)
{
	int32 Index = 0;
	for (const FExtendedLensFile& Calibration : InCalibrationArray)
	{
		// Don't take into account cameras that do not match the camera type (is depth)
		if (Calibration.IsDepthCamera != bIsDepth)
		{
			continue;
		}

		if (Calibration.Name == InName)
		{
			return Index;
		}

		// Only increase index for specified type of camera
		++Index;
	}

	return INDEX_NONE;
}

int32 UFootageCaptureData::GetViewIndexByCameraName(const FString& InName) const
{
	if (CameraCalibrations.IsEmpty())
	{
		int32 Index = INDEX_NONE;

		TArray<FString> Tokens; // See UFootageCaptureData::PopulateCameraNames for camera name format
		InName.ParseIntoArray(Tokens, TEXT(" "));

		if (Tokens.Num() == 2 && Tokens[1].IsNumeric())
		{
			Index = FCString::Atoi(*Tokens[1]);
		}

		return Index;
	}

	// Making sure that the code above returns if the CameraCalibrations array is empty
	ensure(CameraCalibrations.Num() >= 1);

	if (!CameraCalibrations[0])
	{
		return INDEX_NONE;
	}

	int32 Index = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	Index = FindCameraByName(ImageSequences, InName);
	if (Index != INDEX_NONE)
	{
		return Index;
	}

	Index = FindCameraByName(DepthSequences, InName);
	if (Index != INDEX_NONE)
	{
		return Index;
	}
#endif

	// Search for video camera
	Index = CheckCalibrationArray(CameraCalibrations[0]->CameraCalibrations, InName, false);

	if (Index != INDEX_NONE && ImageSequences.IsValidIndex(Index))
	{
		return Index;
	}

	// Search for depth camera
	Index = CheckCalibrationArray(CameraCalibrations[0]->CameraCalibrations, InName, true);

	if (Index != INDEX_NONE && DepthSequences.IsValidIndex(Index))
	{
		return Index;
	}

	const FString AvailableNames = BuildAvailableCalibrationsString();

	UE_LOGF(
		LogCaptureDataCore,
		Warning,
		"Camera name \"%ls\" could not be found among the calibrations of the capture data asset. Available names are %ls",
		*InName,
		*AvailableNames
	);

	return INDEX_NONE;
}

FString UFootageCaptureData::BuildAvailableCalibrationsString() const
{
	// Asking the camera calibration for this information directly would be cleaner but would require API changes we are
	// unable to make during a hotfix, so for now we must live with the duplication.

	FString CalibrationNames = TEXT("[");

	if (!CameraCalibrations.IsEmpty())
	{
		// There remains an assumption that only the first camera calibration is to be used
		const TArray<FExtendedLensFile>& LensFiles = CameraCalibrations[0]->CameraCalibrations;

		if (!LensFiles.IsEmpty())
		{
			CalibrationNames += LensFiles[0].Name;
		}

		for (int32 Index = 1; Index < LensFiles.Num(); ++Index)
		{
			CalibrationNames += FString::Printf(TEXT(", %s"), *LensFiles[Index].Name);
		}
	}

	CalibrationNames += TEXT("]");

	return CalibrationNames;
}

