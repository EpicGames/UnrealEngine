// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureDataCoreModule.h"

#include "CaptureData.h"
#include "CaptureDataLog.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "CaptureDataCoreModule"

void FCaptureDataCoreModule::StartupModule()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().AddRaw(this, &FCaptureDataCoreModule::CheckAssetMigration);
}

void FCaptureDataCoreModule::ShutdownModule()
{

}

/*
Check for MetaHuman assets which need to be migrated due to the source code moving to the Capture Data plugin.
*/
void FCaptureDataCoreModule::CheckAssetMigration()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> Assets;
	
	// Some MetaHuman assets have moved to the Capture Data plugin so find assets which are referencing the old class path
	FARFilter Filter;
	Filter.ClassPaths.Add(FTopLevelAssetPath(FName("/Script/MetaHumanCaptureData"), FName("FootageCaptureData")));
	Filter.ClassPaths.Add(FTopLevelAssetPath(FName("/Script/MetaHumanCaptureData"), FName("MeshCaptureData")));
	Filter.ClassPaths.Add(FTopLevelAssetPath(FName("/Script/MetaHumanCore"), FName("MetaHumanCameraCalibration")));
	Filter.bIncludeOnlyOnDiskAssets = true;

	AssetRegistryModule.Get().GetAssets(Filter, Assets);

	// Prompt user to reload these assets
	if (!Assets.IsEmpty())
	{
		UE_LOGF(LogCaptureDataCore, Display, "Found %d Capture Data assets which need to be updated. Starting update ...", Assets.Num());
		for (FAssetData AssetData : Assets)
		{
			if (UObject* Asset = AssetData.GetAsset())
			{
				Asset->ReloadConfig();
			}
		}
		UE_LOGF(LogCaptureDataCore, Display, "Finished updating Capture Data assets");
	}	
}

IMPLEMENT_MODULE(FCaptureDataCoreModule, CaptureDataCore)

#undef LOCTEXT_NAMESPACE