// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureDataLog.h"

DEFINE_LOG_CATEGORY(LogCaptureDataCore);