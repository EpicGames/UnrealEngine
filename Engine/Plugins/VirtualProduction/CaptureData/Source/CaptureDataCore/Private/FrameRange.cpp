// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameRange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FrameRange)

bool FFrameRange::operator==(const FFrameRange& InOther) const
{
	return (Name == InOther.Name && StartFrame == InOther.StartFrame && EndFrame == InOther.EndFrame);
}

bool FFrameRange::ContainsFrame(int32 InFrame, const TArray<FFrameRange>& InFrameRangeArray)
{
	for (const FFrameRange& FrameRange : InFrameRangeArray)
	{
		if ((FrameRange.StartFrame >= 0 || FrameRange.EndFrame >= 0) &&
			(FrameRange.StartFrame < 0 || InFrame >= FrameRange.StartFrame) && //-V590
			(FrameRange.EndFrame < 0 || InFrame <= FrameRange.EndFrame))
		{
			return true;
		}
	}

	return false;
}

TArray<FFrameRange> PackIntoFrameRanges(TArray<FFrameNumber> InFrameNumbers)
{
	TArray<FFrameRange> FrameRanges;

	if (InFrameNumbers.IsEmpty())
	{
		return FrameRanges;
	}

	// Make sure the supplied frame numbers are in the correct order
	InFrameNumbers.Sort();

	TOptional<FFrameNumber> LastFrameNumber;
	TOptional<FFrameNumber> CurrentRangeStart;

	for (const FFrameNumber FrameNumber : InFrameNumbers)
	{
		if (!CurrentRangeStart.IsSet())
		{
			CurrentRangeStart = FrameNumber;
		}

		if (!LastFrameNumber.IsSet())
		{
			LastFrameNumber = FrameNumber;
		}

		if (FrameNumber - *LastFrameNumber > 1)
		{
			FFrameRange NewRange;
			NewRange.StartFrame = CurrentRangeStart->Value;
			NewRange.EndFrame = LastFrameNumber->Value;
			CurrentRangeStart = FrameNumber;

			FrameRanges.Emplace(MoveTemp(NewRange));
		}

		LastFrameNumber = FrameNumber;
	}

	FFrameRange LastRange;
	LastRange.StartFrame = CurrentRangeStart->Value;
	LastRange.EndFrame = LastFrameNumber->Value;
	FrameRanges.Emplace(MoveTemp(LastRange));

	return FrameRanges;
}
