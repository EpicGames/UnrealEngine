// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageSequencePathChecker.h"

#include "CaptureDataLog.h"

#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "ImageSequencePathChecker"

namespace UE::CaptureData
{
static const FText DialogTitle = LOCTEXT("MissingDataTitle", "Missing Data");
static const FTextFormat DialogMessageFormat = LOCTEXT(
	"ImageSequenceMissingDataMessage",

	"{AssetDisplayName} {NumCaptureDataFootageAssets}|plural(one=asset contains,other=assets contain) "
	"{NumInvalidImageSequences}|plural(one=an image sequence which is,other=image sequences which have) missing data.\n\n"
	"See output log for details."
);

FImageSequencePathChecker::FImageSequencePathChecker(FText InAssetDisplayName) :
	NumCaptureDataFootageAssets(0),
	NumInvalidImageSequences(0),
	AssetDisplayName(MoveTemp(InAssetDisplayName))
{
}

void FImageSequencePathChecker::Check(const UFootageCaptureData& InCaptureData)
{
	++NumCaptureDataFootageAssets;

	const TArray<UFootageCaptureData::FPathAssociation> InvalidImageSequences = InCaptureData.CheckImageSequencePaths();
	NumInvalidImageSequences += InvalidImageSequences.Num();

	if (!InvalidImageSequences.IsEmpty())
	{
		UE_LOGF(
			LogCaptureDataCore,
			Warning,
			"%ls contains image sequence(s) with missing data (see below): %ls",
			*AssetDisplayName.ToString(),
			*InCaptureData.GetPathName()
		);

		for (const UFootageCaptureData::FPathAssociation& InvalidImageSequence : InvalidImageSequences)
		{
			UE_LOGF(
				LogCaptureDataCore,
				Warning,
				"Image sequence path does not exist or is not a folder: %ls (%ls)",
				*InvalidImageSequence.PathOnDisk,
				*InvalidImageSequence.AssetPath
			);
		}
	}
}

void FImageSequencePathChecker::DisplayDialog() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("NumCaptureDataFootageAssets"), NumCaptureDataFootageAssets);
	Args.Add(TEXT("AssetDisplayName"), AssetDisplayName);
	Args.Add(TEXT("NumInvalidImageSequences"), NumInvalidImageSequences);

	const FText DialogMessage = FText::Format(DialogMessageFormat, Args);

	FMessageDialog::Open(EAppMsgType::Ok, DialogMessage, DialogTitle);
}

bool FImageSequencePathChecker::HasError() const
{
	return NumInvalidImageSequences > 0;
}

}

#undef LOCTEXT_NAMESPACE

