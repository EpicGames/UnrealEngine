// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageSequenceUtils.h"

#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImgMediaSource.h"
#include "Internationalization/Regex.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "TakeDirectoryUtils.h"
#include "CaptureDataLog.h"


bool FImageSequenceUtils::GetImageSequencePathAndFilesFromAsset(const UImgMediaSource* InImgSequence, FString& OutFullSequencePath, TArray<FString>& OutImageFiles)
{
	if (InImgSequence == nullptr)
	{
		return false;
	}

	OutFullSequencePath = InImgSequence->GetFullPath();

	return GetImageSequenceFilesFromPath(OutFullSequencePath, OutImageFiles);
}

bool FImageSequenceUtils::GetImageSequenceFilesFromPath(const FString& InFullSequencePath, TArray<FString>& OutImageFiles, bool bInSort)
{
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	IFileManager& FileManager = IFileManager::Get();

	bool bIterateResult = FileManager.IterateDirectory(*InFullSequencePath, [&OutImageFiles, &ImageWrapperModule](const TCHAR* InFilenameOrDirectory, bool bInIsDirectory)
	{
		if (!bInIsDirectory)
		{
			EImageFormat Format = ImageWrapperModule.GetImageFormatFromExtension(*FPaths::GetExtension(InFilenameOrDirectory));
			if (Format != EImageFormat::Invalid)
			{
				OutImageFiles.Add(FPaths::GetCleanFilename(InFilenameOrDirectory));
			}
		}

		return true;
	});

	if (bInSort)
	{
		using namespace UE::CaptureData;

		FRegexPattern Pattern = GetRegexPattern();
		OutImageFiles.Sort([&Pattern](const FString& InLeft, const FString& InRight)
		{
			FString Dummy;
			FString LeftDigits, RightDigits;
			bool bSuccess = ExtractInfoFromFileName(Pattern, InLeft, Dummy, LeftDigits, Dummy);
			bSuccess = bSuccess && ExtractInfoFromFileName(Pattern, InRight, Dummy, RightDigits, Dummy);

			if (bSuccess)
			{
				return FCString::Atoi(*LeftDigits) < FCString::Atoi(*RightDigits);
			}

			return InLeft < InRight;
		});
	}

	return !OutImageFiles.IsEmpty() && bIterateResult;
}

bool FImageSequenceUtils::GetImageSequenceInfoFromAsset(const class UImgMediaSource* InImgSequence, FIntVector2& OutDimensions, int32& OutNumImages)
{
	if (InImgSequence == nullptr)
	{
		UE_LOGF(LogCaptureDataCore, Warning, "GetImageSequenceInfoFromAsset: InImgSequence is null.");
		return false;
	}

	return GetImageSequenceInfoFromPath(InImgSequence->GetFullPath(), OutDimensions, OutNumImages);
}

bool FImageSequenceUtils::GetImageSequenceInfoFromPath(const FString& InFullSequencePath, FIntVector2& OutDimensions, int32& OutNumImages)
{
	TArray<FString> ImageFiles;
	bool bFoundImages = GetImageSequenceFilesFromPath(InFullSequencePath, ImageFiles, false);

	if (!bFoundImages)
	{
		UE_LOGF(LogCaptureDataCore, Warning, "GetImageSequenceInfoFromPath: No image files found in '%ls'.", *InFullSequencePath);
		return false;
	}

	OutNumImages = ImageFiles.Num();

	if (ImageFiles.IsEmpty())
	{
		UE_LOGF(LogCaptureDataCore, Warning, "GetImageSequenceInfoFromPath: Image file list is empty for '%ls'.", *InFullSequencePath);
		return false;
	}

	const FString SampleImagePath = InFullSequencePath / ImageFiles[0];

	TArray<uint8> RawFileData;
	if (!FFileHelper::LoadFileToArray(RawFileData, *SampleImagePath))
	{
		UE_LOGF(LogCaptureDataCore, Warning, "GetImageSequenceInfoFromPath: Failed to load file '%ls'.", *SampleImagePath);
		return false;
	}

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(RawFileData.GetData(), RawFileData.Num());
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);

	if (!ImageWrapper.IsValid() || !ImageWrapper->SetCompressed(RawFileData.GetData(), RawFileData.Num()))
	{
		UE_LOGF(LogCaptureDataCore, Warning, "GetImageSequenceInfoFromPath: Failed to decode image '%ls'.", *SampleImagePath);
		return false;
	}

	OutDimensions.X = ImageWrapper->GetWidth();
	OutDimensions.Y = ImageWrapper->GetHeight();
	return true;
}

// We consider that the image sequence is valid if it has at least one image
bool FImageSequenceUtils::IsImageSequenceValid(const UImgMediaSource* InImgSequence)
{
	if (!InImgSequence)
	{
		return false;
	}

	FString FullImageSequencePath = InImgSequence->GetFullPath();

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	IFileManager& FileManager = IFileManager::Get();

	bool bImageFound = false;

	FileManager.IterateDirectory(*FullImageSequencePath, [&bImageFound, &ImageWrapperModule](const TCHAR* InFilenameOrDirectory, bool bInIsDirectory)
	{
		if (bInIsDirectory)
		{
			return true;
		}

		EImageFormat Format = ImageWrapperModule.GetImageFormatFromExtension(*FPaths::GetExtension(InFilenameOrDirectory));
		if (Format == EImageFormat::Invalid)
		{
			return true;
		}

		bImageFound = true;

		// Break the loop when we find the image
		return false;
	});

	return bImageFound;
}