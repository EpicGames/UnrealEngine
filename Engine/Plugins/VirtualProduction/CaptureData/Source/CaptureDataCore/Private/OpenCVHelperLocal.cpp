// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenCVHelperLocal.h"
#include "Math/TransformCalculus3D.h"

void FOpenCVHelperLocal::ConvertCoordinateSystem(FTransform& Transform, const EAxis SrcXInDstAxis, const EAxis SrcYInDstAxis, const EAxis SrcZInDstAxis)
{
	// Unreal Engine:
	//   Front : X
	//   Right : Y
	//   Up    : Z
	//
	// OpenCV:
	//   Front : Z
	//   Right : X
	//   Up    : Yn

	FMatrix M12 = FMatrix::Identity;

	M12.SetColumn(0, UnitVectorFromAxisEnum(SrcXInDstAxis));
	M12.SetColumn(1, UnitVectorFromAxisEnum(SrcYInDstAxis));
	M12.SetColumn(2, UnitVectorFromAxisEnum(SrcZInDstAxis));

	Transform.SetFromMatrix(M12.GetTransposed() * Transform.ToMatrixWithScale() * M12);
}

void FOpenCVHelperLocal::ConvertUnrealToOpenCV(FTransform& Transform)
{
	ConvertCoordinateSystem(Transform, EAxis::Y, EAxis::Zn, EAxis::X);
}

void FOpenCVHelperLocal::ConvertOpenCVToUnreal(FTransform& Transform)
{
	ConvertCoordinateSystem(Transform, EAxis::Z, EAxis::X, EAxis::Yn);
}

FVector FOpenCVHelperLocal::ConvertUnrealToOpenCV(const FVector& Vector)
{
	return FVector(Vector.Y, -Vector.Z, Vector.X);
}

FVector FOpenCVHelperLocal::ConvertOpenCVToUnreal(const FVector& Vector)
{
	return FVector(Vector.Z, Vector.X, -Vector.Y);
}

void FOpenCVHelperLocal::ConvertOpenCVToUnreal(const FMatrix& InRotationOpenCV, const FVector& InTranslationOpenCV, FRotator& OutRotatorUE, FVector& OutTranslationUE)
{
	// this function converts a rotation and translation from OpenCV coordinate frame, where we have
	// right-handed coordinate system, x right, y down to
	// UE coordinate system, where we have left-handed coordinate system, y right, z up

	FVector XAxisUE = FVector(InRotationOpenCV.M[2][2], InRotationOpenCV.M[2][0], -InRotationOpenCV.M[2][1]); // comes from Z axis in rotation matrix 
	FVector YAxisUE = FVector(InRotationOpenCV.M[0][2], InRotationOpenCV.M[0][0], -InRotationOpenCV.M[0][1]); // comes from X axis in rotation matrix
	FMatrix RotationUE = FRotationMatrix::MakeFromXY(XAxisUE, YAxisUE);

	// again swap the elements in the position to get the result in UE coordinate space
	OutTranslationUE = FVector(InTranslationOpenCV.Z, InTranslationOpenCV.X, -InTranslationOpenCV.Y);
	OutRotatorUE = TransformConverter<FRotator>::Convert(RotationUE);
}
