// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureData.h"

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "UObject/Class.h"

DEFINE_LOG_CATEGORY_STATIC(LogFootageCaptureMetadataSpec, Log, All);

BEGIN_DEFINE_SPEC(FootageCaptureMetadataSpec, "MetaHuman.Capture.FootageCaptureMetadata"
	, EAutomationTestFlags_ApplicationContextMask
	| EAutomationTestFlags::ProductFilter)
	FFootageCaptureMetadata Metadata;
	TMap<FString, EFootageDeviceClass> ModelNameToDeviceClass;
END_DEFINE_SPEC(FootageCaptureMetadataSpec)
void FootageCaptureMetadataSpec::Define()
{	
	Describe("SetDeviceClass", [this]()
	{
		// https://theapplewiki.com/wiki/Models
		ModelNameToDeviceClass.Reset();
		ModelNameToDeviceClass.Add("iPhone10,1", EFootageDeviceClass::iPhone11OrEarlier); // iPhone 8 (1)
		ModelNameToDeviceClass.Add("iPhone10,2", EFootageDeviceClass::iPhone11OrEarlier); // iPhone 8 Plus (1)
		ModelNameToDeviceClass.Add("iPhone10,3", EFootageDeviceClass::iPhone11OrEarlier); // iPhone X (1)
		ModelNameToDeviceClass.Add("iPhone10,4", EFootageDeviceClass::iPhone11OrEarlier); // iPhone 8 (2)
		ModelNameToDeviceClass.Add("iPhone10,5", EFootageDeviceClass::iPhone11OrEarlier); // iPhone Plus (2)
		ModelNameToDeviceClass.Add("iPhone10,6", EFootageDeviceClass::iPhone11OrEarlier); // iPhone X (2)
		ModelNameToDeviceClass.Add("iPhone11,2", EFootageDeviceClass::iPhone11OrEarlier); // iPhone XS
		ModelNameToDeviceClass.Add("iPhone11,4", EFootageDeviceClass::iPhone11OrEarlier); // iPhone XS Max (1)
		ModelNameToDeviceClass.Add("iPhone11,6", EFootageDeviceClass::iPhone11OrEarlier); // iPhone XS Max (2)
		ModelNameToDeviceClass.Add("iPhone11,8", EFootageDeviceClass::iPhone11OrEarlier); // iPhone XR
		ModelNameToDeviceClass.Add("iPhone12,1", EFootageDeviceClass::iPhone11OrEarlier); // iPhone 11
		ModelNameToDeviceClass.Add("iPhone12,3", EFootageDeviceClass::iPhone11OrEarlier); // iPhone 11 Pro
		ModelNameToDeviceClass.Add("iPhone12,5", EFootageDeviceClass::iPhone11OrEarlier); // iPhone 11 Pro Max
		ModelNameToDeviceClass.Add("iPhone12,8", EFootageDeviceClass::OtheriOSDevice);    // iPhone SE 2
		ModelNameToDeviceClass.Add("iPhone13,1", EFootageDeviceClass::iPhone12);          // iPhone 12 Mini
		ModelNameToDeviceClass.Add("iPhone13,2", EFootageDeviceClass::iPhone12);          // iPhone 12
		ModelNameToDeviceClass.Add("iPhone13,3", EFootageDeviceClass::iPhone12);          // iPhone 12 Pro
		ModelNameToDeviceClass.Add("iPhone13,4", EFootageDeviceClass::iPhone12);          // iPhone 12 Pro Max
		ModelNameToDeviceClass.Add("iPhone14,2", EFootageDeviceClass::iPhone13);          // iPhone 13 Pro
		ModelNameToDeviceClass.Add("iPhone14,3", EFootageDeviceClass::iPhone13);          // iPhone 13 Pro Max
		ModelNameToDeviceClass.Add("iPhone14,4", EFootageDeviceClass::iPhone13);          // iPhone 13 Mini
		ModelNameToDeviceClass.Add("iPhone14,5", EFootageDeviceClass::iPhone13);          // iPhone 13
		ModelNameToDeviceClass.Add("iPhone14,6", EFootageDeviceClass::OtheriOSDevice);    // iPhone SE 3
		ModelNameToDeviceClass.Add("iPhone14,7", EFootageDeviceClass::iPhone14OrLater);   // iPhone 14
		ModelNameToDeviceClass.Add("iPhone14,8", EFootageDeviceClass::iPhone14OrLater);   // iPhone 14 Plus
		ModelNameToDeviceClass.Add("iPhone15,2", EFootageDeviceClass::iPhone14OrLater);   // iPhone 14 Pro
		ModelNameToDeviceClass.Add("iPhone15,3", EFootageDeviceClass::iPhone14OrLater);   // iPhone 14 Pro Max
		ModelNameToDeviceClass.Add("iPhone15,4", EFootageDeviceClass::iPhone14OrLater);   // iPhone 15
		ModelNameToDeviceClass.Add("iPhone15,5", EFootageDeviceClass::iPhone14OrLater);   // iPhone 15 Plus
		ModelNameToDeviceClass.Add("iPhone16,1", EFootageDeviceClass::iPhone14OrLater);   // iPhone 15 Pro
		ModelNameToDeviceClass.Add("iPhone16,2", EFootageDeviceClass::iPhone14OrLater);   // iPhone 15 Pro Max
		ModelNameToDeviceClass.Add("iPhone99,1", EFootageDeviceClass::iPhone14OrLater);   // Future iPhone
		ModelNameToDeviceClass.Add("iPhone999,9", EFootageDeviceClass::iPhone14OrLater);  // Distant Future iPhone
		ModelNameToDeviceClass.Add("iPhone", EFootageDeviceClass::OtheriOSDevice);        // Invalid iPhone Model Number
		ModelNameToDeviceClass.Add("iPhone1,2,3", EFootageDeviceClass::OtheriOSDevice);   // Invalid iPhone Model Number
		
		ModelNameToDeviceClass.Add("iPad8,1", EFootageDeviceClass::OtheriOSDevice);       // iPad Pro 11-inch 1
		ModelNameToDeviceClass.Add("iPad11,3", EFootageDeviceClass::OtheriOSDevice);      // iPad Air 3
		ModelNameToDeviceClass.Add("iPad13,11", EFootageDeviceClass::OtheriOSDevice);     // iPad Pro 12.9-inch 5
		ModelNameToDeviceClass.Add("iPad", EFootageDeviceClass::OtheriOSDevice);          // Invalid iPad Model Number

		ModelNameToDeviceClass.Add("StereoHMC", EFootageDeviceClass::StereoHMC);          // Stereo HMC

		ModelNameToDeviceClass.Add("iTablet", EFootageDeviceClass::Unspecified);          // Invalid Model Number
		ModelNameToDeviceClass.Add("NotAnIPhone", EFootageDeviceClass::Unspecified);      // Invalid Model Number

		for (const TPair<FString, EFootageDeviceClass>& Mapping : ModelNameToDeviceClass)
		{
			const FString ModelName = Mapping.Key;
			const EFootageDeviceClass Expected = Mapping.Value;
			const FString ExpectedName = UEnum::GetDisplayValueAsText(Expected).ToString();
			It(FString::Printf(TEXT("should set DeviceClass to '%s' when DeviceModel is '%s'"), *ExpectedName, *ModelName), [this, ModelName, Expected]
			{
				Metadata.SetDeviceClass(ModelName);
				const EFootageDeviceClass Actual = Metadata.DeviceClass;
				TestEqual("DeviceClass", Actual, Expected);
			});
		}
	});
}