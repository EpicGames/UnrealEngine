// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameRange.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::MetaHuman::Private
{

static TArray<FFrameRange> FrameRangeForEachFrameNumber(const TArray<FFrameNumber> InFrameNumbers)
{
	TArray<FFrameRange> FrameRanges;
	FrameRanges.Reserve(InFrameNumbers.Num());

	for (const FFrameNumber FrameNumber : InFrameNumbers)
	{
		FFrameRange FrameRange;
		FrameRange.StartFrame = FrameNumber.Value;
		FrameRange.EndFrame = FrameNumber.Value;
		FrameRanges.Add(MoveTemp(FrameRange));
	}

	return FrameRanges;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_Empty,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.Empty",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_Empty::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers;
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);
	UTEST_TRUE("Frame ranges is empty", FrameRanges.IsEmpty());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_SingleBlock,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.SingleBlock",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_SingleBlock::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers = { 1, 2, 3 };
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);

	TArray<FFrameRange> ExpectedFrameRanges;
	FFrameRange FrameRange;
	FrameRange.StartFrame = 1;
	FrameRange.EndFrame = 3;
	ExpectedFrameRanges.Emplace(MoveTemp(FrameRange));

	UTEST_EQUAL("Frame ranges", FrameRanges, ExpectedFrameRanges);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_TwoBlocks,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.TwoBlocks",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_TwoBlocks::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers = { 1, 2, 3, 5, 6, 7 };
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);

	TArray<FFrameRange> ExpectedFrameRanges;
	FFrameRange FirstBlock;
	FirstBlock.StartFrame = 1;
	FirstBlock.EndFrame = 3;
	ExpectedFrameRanges.Emplace(MoveTemp(FirstBlock));

	FFrameRange SecondBlock;
	SecondBlock.StartFrame = 5;
	SecondBlock.EndFrame = 7;
	ExpectedFrameRanges.Emplace(MoveTemp(SecondBlock));

	UTEST_EQUAL("Frame ranges", FrameRanges, ExpectedFrameRanges);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_TwoBlocksUnsorted,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.TwoBlocksUnsorted",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_TwoBlocksUnsorted::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers = { 7, 5, 3, 6, 2, 1 };
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);

	TArray<FFrameRange> ExpectedFrameRanges;
	FFrameRange FirstBlock;
	FirstBlock.StartFrame = 1;
	FirstBlock.EndFrame = 3;
	ExpectedFrameRanges.Emplace(MoveTemp(FirstBlock));

	FFrameRange SecondBlock;
	SecondBlock.StartFrame = 5;
	SecondBlock.EndFrame = 7;
	ExpectedFrameRanges.Emplace(MoveTemp(SecondBlock));

	UTEST_EQUAL("Frame ranges", FrameRanges, ExpectedFrameRanges);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_NoBlocks,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.NoBlocks",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_NoBlocks::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers = { 1, 3, 5, 7, 9, 11 };
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);

	TArray<FFrameRange> ExpectedFrameRanges = FrameRangeForEachFrameNumber(FrameNumbers);
	UTEST_EQUAL("Frame ranges", FrameRanges, ExpectedFrameRanges);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_NegativeSingleBlock,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.NegativeSingleBlock",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_NegativeSingleBlock::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers = { -3, -2, -1 };
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);

	TArray<FFrameRange> ExpectedFrameRanges;
	FFrameRange FrameRange;
	FrameRange.StartFrame = -3;
	FrameRange.EndFrame = -1;
	ExpectedFrameRanges.Emplace(MoveTemp(FrameRange));

	UTEST_EQUAL("Frame ranges", FrameRanges, ExpectedFrameRanges);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_NegativeSingleBlockUnsorted,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.NegativeSingleBlockUnsorted",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_NegativeSingleBlockUnsorted::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers = { -1, -2, -3 };
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);

	TArray<FFrameRange> ExpectedFrameRanges;
	FFrameRange FrameRange;
	FrameRange.StartFrame = -3;
	FrameRange.EndFrame = -1;
	ExpectedFrameRanges.Emplace(MoveTemp(FrameRange));

	UTEST_EQUAL("Frame ranges", FrameRanges, ExpectedFrameRanges);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_NegativeTwoBlocksUnsorted,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.NegativeTwoBlocksUnsorted",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_NegativeTwoBlocksUnsorted::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers = { -1, -2, -3, -6, -7, -8 };
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);

	TArray<FFrameRange> ExpectedFrameRanges;
	FFrameRange FirstBlock;
	FirstBlock.StartFrame = -8;
	FirstBlock.EndFrame = -6;
	ExpectedFrameRanges.Emplace(MoveTemp(FirstBlock));

	FFrameRange SecondBlock;
	SecondBlock.StartFrame = -3;
	SecondBlock.EndFrame = -1;
	ExpectedFrameRanges.Emplace(MoveTemp(SecondBlock));

	UTEST_EQUAL("Frame ranges", FrameRanges, ExpectedFrameRanges);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FPackIntoFrameRangesTest_MixturePostiveNegativeWithDuplicatesUnsorted,
	"MetaHuman.Capture.CaptureDataCore.PackIntoFrameRanges.MixturePostiveNegativeWithDuplicatesUnsorted",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FPackIntoFrameRangesTest_MixturePostiveNegativeWithDuplicatesUnsorted::RunTest(const FString& InParameters)
{
	TArray<FFrameNumber> FrameNumbers = { 8, -2, 7, -1, 6, -3, 9, -3, 7 };
	TArray<FFrameRange> FrameRanges = PackIntoFrameRanges(FrameNumbers);

	TArray<FFrameRange> ExpectedFrameRanges;
	FFrameRange FirstBlock;
	FirstBlock.StartFrame = -3;
	FirstBlock.EndFrame = -1;
	ExpectedFrameRanges.Emplace(MoveTemp(FirstBlock));

	FFrameRange SecondBlock;
	SecondBlock.StartFrame = 6;
	SecondBlock.EndFrame = 9;
	ExpectedFrameRanges.Emplace(MoveTemp(SecondBlock));

	UTEST_EQUAL("Frame ranges", FrameRanges, ExpectedFrameRanges);

	return true;
}

}

#endif // WITH_DEV_AUTOMATION_TESTS
