// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "EditorFramework/AssetImportData.h"

#include "IMediaTextureSample.h"

#include "CameraCalibration.generated.h"

#define UE_API CAPTUREDATACORE_API


USTRUCT(BlueprintType)
struct FExtendedLensFile
{
	GENERATED_BODY()


public:
	UPROPERTY(EditAnywhere, Category = Calibration)
	FString Name;

	UPROPERTY(EditAnywhere, Category = Calibration)
	bool IsDepthCamera{};

	UPROPERTY(EditAnywhere, Category = Calibration)
	TObjectPtr<class ULensFile> LensFile{};
};

USTRUCT(BlueprintType)
struct FStereoPair
{
	GENERATED_BODY()


public:
	UPROPERTY(EditAnywhere, Category = Calibration)
	uint32 CameraIndex1 {};

	UPROPERTY(EditAnywhere, Category = Calibration)
	uint32 CameraIndex2 {};
};

struct FCameraCalibration
{
public:

	enum EType
	{
		Video = 0,
		Depth,

		Unknown,
	};
	
	FString CameraId;
	EType CameraType;

	FVector2D ImageSize = FVector2D::Zero();
	FVector2D FocalLength = FVector2D::Zero();
	FVector2D PrincipalPoint = FVector2D::Zero();
	FVector2D FocalLengthNormalized = FVector2D::Zero();
	FVector2D PrincipalPointNormalized = FVector2D::Zero();

	double K1 = 0;
	double K2 = 0;
	double P1 = 0;
	double P2 = 0;
	double K3 = 0;
	double K4 = 0;
	double K5 = 0;
	double K6 = 0;
	FMatrix Transform;
	EMediaOrientation Orientation = EMediaOrientation::Original;
};

/** Camera Calibration Asset
*
*   Contains the parameters for calibrating the camera
*   used in footage for MetaHuman Identity and Performance assets.
* 
**/
UCLASS(MinimalAPI, BlueprintType)
class UCameraCalibration : public UObject
{
	GENERATED_BODY()

public:
	//~Begin UObject interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	//~End UObject interface

	UPROPERTY(EditAnywhere, Category = Calibration)
	TArray<FExtendedLensFile> CameraCalibrations;

	UPROPERTY(EditAnywhere, Category = Calibration)
	TArray<FStereoPair> StereoPairs;

#if WITH_EDITORONLY_DATA
	/** Importing data and options used for importing mhaical files */
	UPROPERTY(VisibleAnywhere, Instanced, Category = "Import Settings")
	TObjectPtr<UAssetImportData> AssetImportData;
#endif // WITH_EDITORONLY_DATA

	UE_API bool ConvertToTrackerNodeCameraModels(TArray<FCameraCalibration>& OutCalibrations,
										  TArray<TPair<FString, FString>>& OutStereoReconstructionPairs) const;

	UE_API bool ConvertFromTrackerNodeCameraModels(const TArray<FCameraCalibration>& InCalibrations, bool bInUsingUnrealCoordinateSystem = false);
	UE_API bool ConvertFromTrackerNodeCameraModels(const TArray<FCameraCalibration>& InCalibrations, const TMap<FString, FString>& InLensAssetNamesMap, bool bInUsingUnrealCoordinateSystem = false);

	UE_API int32 GetCalibrationIndexByName(const FString& InName) const;

private:

	UE_API void CreateLensFileForCalibration(const FCameraCalibration& InCalibration, const FString& InAssetName, bool bInUsingUnrealCoordinateSystem);
	UE_API void AddStereoPair();
	UE_API FString BuildAvailableCalibrationsString() const;
	
	EMediaOrientation CameraOrientation;
};

#undef UE_API
