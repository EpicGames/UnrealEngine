// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "IntVectorTypes.h"
#include "Misc/Timecode.h"
#include "Misc/Paths.h"
#include "Engine/StaticMesh.h"
#include "FrameRange.h"
#include "Templates/ValueOrError.h"

#include "Misc/EnumClassFlags.h"

#include "CaptureData.generated.h"

#define UE_API CAPTUREDATACORE_API

/////////////////////////////////////////////////////
// UCaptureData

enum class ECaptureDataInitializedCheck : uint8
{
	ImageSequences = 1 << 0,
	DepthSequences = 1 << 1,
	Calibrations = 1 << 2,
	Metadata = 1 << 3,

	Full = ImageSequences | DepthSequences | Calibrations | Metadata,
	ImageSequencesOnly = ImageSequences
};

ENUM_CLASS_FLAGS(ECaptureDataInitializedCheck);

// Delegate called when something changes in the capture data that others should know about
DECLARE_MULTICAST_DELEGATE(FOnCaptureDataInternalsChanged)

UCLASS(Abstract, MinimalAPI, BlueprintType)
class UCaptureData : public UObject
{
	GENERATED_BODY()

public:
	
	enum UE_DEPRECATED(5.8, "EInitializedCheck has been moved to global scope. Please use the ECaptureDataInitializedCheck::.") EInitializedCheck
	{
		Full = 0,
		ImageSequencesOnly,
	};

	/** Returns true is the capture data is fully initialized with all required information present */
	virtual bool IsInitialized(ECaptureDataInitializedCheck InInitializedCheck) const PURE_VIRTUAL(UCaptureData::IsInitialized, return false;);

	FOnCaptureDataInternalsChanged& OnCaptureDataInternalsChanged()
	{
		return OnCaptureDataInternalsChangedDelegate;
	}

#if WITH_EDITOR

	//~Begin UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PostTransacted(const FTransactionObjectEvent& InTransactionEvent) override;
	//~End UObject interface

#endif

protected:
	/** Notify that something internal to the capture data changed */
	void NotifyInternalsChanged();

private:
	FOnCaptureDataInternalsChanged OnCaptureDataInternalsChangedDelegate;
};

/** Capture Data (Mesh) Asset
*
*   An asset that contains the Mesh data representing a facial
*   expression (Pose), that can be used in MetaHuman Identity
*   to generate a Skeletal Mesh or a full MetaHuman resembling
*   a real person or a sculpted character.
*
*   If pose data represents a real person, the resulting
*   Skeletal Mesh can be used to generate animation from footage
*   of that person in MetaHuman Performance asset
*
**/
UCLASS(MinimalAPI)
class UMeshCaptureData : public UCaptureData
{
	GENERATED_BODY()

public:
	// UCaptureData interface
	UE_API virtual bool IsInitialized(ECaptureDataInitializedCheck InInitializedCheck) const override;

public:
	/** Gets the data in the format expected by the face fitting API */
	UE_API void GetDataForConforming(const FTransform& InTransform, TArray<float>& OutVertices, TArray<int32>& OutTriangles) const;

	// The target mesh for conforming. This can be either a Static or SkeletalMesh
	UPROPERTY(EditAnywhere, Category = "Capture", Meta = (AllowedClasses = "/Script/Engine.StaticMesh, /Script/Engine.SkeletalMesh"))
	TObjectPtr<class UObject> TargetMesh;
};

/////////////////////////////////////////////////////
// UFootageCaptureData

UENUM()
enum class EFootageDeviceClass : uint8
{
	Unspecified,
	iPhone11OrEarlier	UMETA(DisplayName = "iPhone 11 or earlier"),
	iPhone12			UMETA(DisplayName = "iPhone 12"),
	iPhone13			UMETA(DisplayName = "iPhone 13"),
	iPhone14OrLater		UMETA(DisplayName = "iPhone 14 or later"),
	OtheriOSDevice		UMETA(DisplayName = "Other iOS device"),
	StereoHMC			UMETA(DisplayName = "Stereo HMC"),
};

USTRUCT(BlueprintType)
struct FFootageCaptureMetadata
{
	GENERATED_BODY()

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Redundant property"))
	int32 Width_DEPRECATED = 0;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Redundant property"))
	int32 Height_DEPRECATED = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata")
	double FrameRate = 0.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata")
	EFootageDeviceClass DeviceClass = EFootageDeviceClass::Unspecified;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Property has been renamed to Device Class"))
	EFootageDeviceClass DeviceModel_DEPRECATED = EFootageDeviceClass::Unspecified;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, DisplayName = "Device Model", Category = "Metadata")
	FString DeviceModelName;

	/** Sets the DeviceClass property parsing the model string accordingly */
	UE_API void SetDeviceClass(const FString& InDeviceModel);

private:
	struct FIosDeviceVersion
	{
		uint16 Major;
		uint16 Minor;

		FIosDeviceVersion(uint16 Major, uint16 Minor)
			: Major(Major), Minor(Minor)
		{}
	};
	UE_API TOptional<FIosDeviceVersion> ParseIosDeviceVersion(const FString& Prefix, const FString& ModelName);
	UE_API EFootageDeviceClass IPhoneDeviceClass(TOptional<FIosDeviceVersion> IosDeviceVersion);
};

USTRUCT(BlueprintType)
struct FFootageCaptureView
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View")
	TObjectPtr<class UImgMediaSource> ImageSequence;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View")
	bool bImageTimecodePresent = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View")
	FTimecode ImageTimecode = FTimecode(0, 0, 0, 0, false);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View")
	FFrameRate ImageTimecodeRate = FFrameRate(30, 1);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View")
	TObjectPtr<class UImgMediaSource> DepthSequence;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View")
	bool bDepthTimecodePresent = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View")
	FTimecode DepthTimecode = FTimecode(0, 0, 0, 0, false);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "View")
	FFrameRate DepthTimecodeRate = FFrameRate(30, 1);
};

UENUM()
enum class ETimecodeAlignment : int32
{
	None            UMETA(DisplayName = "None"),
	Absolute        UMETA(DisplayName = "Absolute"),
	Relative        UMETA(DisplayName = "Relative"),
};

/**
 * Capture Data (Footage) Asset
 *
 *  An asset that contains the footage data showing facial
 *  expressions (Poses) that can be used by MetaHuman Identity
 *  Asset toolkit to generate a Skeletal Mesh or a MetaHuman
 *  resembling a real person or a sculpted character.
 *
 *  The resulting Skeletal Mesh resembling the actor from the
 *  footage can be used for generating animation from footage
 *  of that person in MetaHuman Performance asset
 *
**/
UCLASS(MinimalAPI)
class UFootageCaptureData : public UCaptureData
{
	GENERATED_BODY()

public:

	using FVerifyResult = TValueOrError<void, FString>;

	//~Begin UCaptureData interface
	UE_API virtual bool IsInitialized(ECaptureDataInitializedCheck InInitializedCheck) const override;
	//~End UCaptureData interface

	//~Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~End UObject interface

public:

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Views are deprecated. Please use Image and Depth sequences instead."))
	TArray<FFootageCaptureView> Views_DEPRECATED;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture")
	TArray<TObjectPtr<class UImgMediaSource>> ImageSequences;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture")
	TArray<TObjectPtr<class UImgMediaSource>> DepthSequences;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture")
	TArray<TObjectPtr<class USoundWave>> AudioTracks;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Audios is deprecated. Please use AudioTracks instead."))
	TArray<TObjectPtr<class USoundWave>> Audios_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Audio is deprecated. Please use AudioTracks instead."))
	TObjectPtr<class USoundWave> Audio_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "AudioTimecodePresent is deprecated."))
	bool bAudioTimecodePresent_DEPRECATED = false;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "AudioTimecode is deprecated."))
	FTimecode AudioTimecode_DEPRECATED = FTimecode(0, 0, 0, 0, false);

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "AudioTimecodeRate is deprecated."))
	FFrameRate AudioTimecodeRate_DEPRECATED = FFrameRate(30, 1);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture")
	TArray<TObjectPtr<class UCameraCalibration>> CameraCalibrations;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture")
	FFootageCaptureMetadata Metadata;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture", DisplayName = "Excluded Frames")
	TArray<FFrameRange> CaptureExcludedFrames;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "CameraCalibration is deprecated. Please use CameraCalibrations instead."))
	TObjectPtr<class UCameraCalibration> CameraCalibration_DEPRECATED;

public:

	/** Gets the resolution of the color channel */
	UE_API FIntPoint GetFootageColorResolution() const;

	UE_API void GetFrameRanges(const FFrameRate& InTargetRate, ETimecodeAlignment InTimecodeAlignment, bool bInIncludeAudio, TMap<TWeakObjectPtr<UObject>, TRange<FFrameNumber>>& OutMediaFrameRanges, TRange<FFrameNumber>& OutProcessingFrameRange, TRange<FFrameNumber>& OutMaximumFrameRange) const;

	static UE_API TRange<FFrameNumber> GetAudioFrameRange(const FFrameRate& InTargetRate, ETimecodeAlignment InTimecodeAlignment, USoundWave* InMedia, const FTimecode& InMediaTimecode, const FFrameRate& InMediaTimecodeRate);

	/** Functions for getting the effective timecode info on a track - note some tracks may not have a timecode set
	*   in which case their effective timecode is defined by others tracks that do have a timecode set.
	*/
	UE_API FTimecode GetEffectiveImageTimecode(int32 InView) const;
	UE_API FFrameRate GetEffectiveImageTimecodeRate(int32 InView) const;

	UE_API FTimecode GetEffectiveDepthTimecode(int32 InView) const;
	UE_API FFrameRate GetEffectiveDepthTimecodeRate(int32 InView) const;

	UE_API FTimecode GetEffectiveAudioTimecode() const;
	UE_API FFrameRate GetEffectiveAudioTimecodeRate() const;

	UE_API FVerifyResult VerifyData(ECaptureDataInitializedCheck InInitializedCheck) const;

	struct FPathAssociation
	{
		FPathAssociation(const FString& InPathOnDisk, const FString& InAssetPath);

		FString PathOnDisk;
		FString AssetPath;
	};

	/** Check the existence of the parent directory for all image sequences and return a list of those which fail the test. */
	[[nodiscard]] UE_API TArray<FPathAssociation> CheckImageSequencePaths() const;

	/** List of all RGB cameras (views) in the footage capture data. Ensures InOutCamera is a valid camera name. */
	static UE_API void PopulateCameraNames(UFootageCaptureData* InFootageCaptureData, FString& InOutCamera, TArray<TSharedPtr<FString>>& OutCameraNames);

	UE_API int32 GetViewIndexByCameraName(const FString& InName) const;

private:
	/** Checks if camera calibration and views are valid. */
	UE_API FVerifyResult ViewsContainsValidData(ECaptureDataInitializedCheck InInitializedCheck) const;
	UE_API FVerifyResult MetadataContainsValidData() const;
	UE_API FVerifyResult CalibrationContainsValidData() const;

	UE_API void GetDefaultTimecodeInfo(FTimecode& OutTimecode, FFrameRate& OutFrameRate) const;
	UE_API FString BuildAvailableCalibrationsString() const;
};

#undef UE_API
