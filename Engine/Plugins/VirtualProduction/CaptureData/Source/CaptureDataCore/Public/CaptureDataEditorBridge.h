// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;

namespace UE::CaptureManager
{

class ICaptureDataEditorBridge : public IModuleInterface
{
public:
	virtual void DeferMarkDirty(TWeakObjectPtr<UObject> InObject) = 0;
};

} // namespace UE::CaptureManager

