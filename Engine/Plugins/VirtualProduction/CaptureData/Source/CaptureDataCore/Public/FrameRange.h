// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "FrameRange.generated.h"

#define UE_API CAPTUREDATACORE_API

USTRUCT(BlueprintType)
struct FFrameRange
{

public:

	GENERATED_BODY()

	UE_API bool operator==(const FFrameRange& InOther) const;

	UPROPERTY(EditAnywhere, Category = "Frame Range")
	FString Name = TEXT("Unnamed");

	UPROPERTY(EditAnywhere, Category = "Frame Range")
	int32 StartFrame = -1;

	UPROPERTY(EditAnywhere, Category = "Frame Range")
	int32 EndFrame = -1;

	static UE_API bool ContainsFrame(int32 InFrame, const TArray<FFrameRange>& InFrameRangeArray);
};

UENUM()
enum class EFrameRangeType : uint8
{
	UserExcluded         UMETA(DisplayName = "User Excluded Frame"),
	ProcessingExcluded   UMETA(DisplayName = "Processing Excluded Frame"),
	CaptureExcluded      UMETA(DisplayName = "Capture Excluded Frame"),
	RateMatchingExcluded UMETA(DisplayName = "Rate Matching Excluded Frame"),
	None
};

typedef TMap<EFrameRangeType, TArray<FFrameRange>> FFrameRangeMap;

/** Packs a list of frame numbers into a sorted array of frame ranges. Essentially collapsing consecutive frame numbers into blocks */
CAPTUREDATACORE_API TArray<FFrameRange> PackIntoFrameRanges(TArray<FFrameNumber> InFrameNumbers);

#undef UE_API
