// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureData.h"

#include "Internationalization/Internationalization.h"

#define UE_API CAPTUREDATACORE_API

namespace UE::CaptureData
{

class FImageSequencePathChecker
{
public:
	UE_API explicit FImageSequencePathChecker(FText InAssetDisplayName);

	UE_API void Check(const UFootageCaptureData& InCaptureData);
	UE_API void DisplayDialog() const;
	UE_API bool HasError() const;

private:
	int32 NumCaptureDataFootageAssets;
	int32 NumInvalidImageSequences;
	FText AssetDisplayName;
};

}

#undef UE_API
