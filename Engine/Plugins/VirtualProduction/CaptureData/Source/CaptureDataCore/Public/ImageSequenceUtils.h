// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API CAPTUREDATACORE_API

/** Utility functions for getting information about image sequences. */
class FImageSequenceUtils
{
public:
	/** Get image sequence path and list of files from image media source asset. */
	static UE_API bool GetImageSequencePathAndFilesFromAsset(const class UImgMediaSource* InImgSequence, FString& OutFullSequencePath, TArray<FString>& OutImageFiles);

	/** Get list of image file paths from the image sequence path. */
	static UE_API bool GetImageSequenceFilesFromPath(const FString& InFullSequencePath, TArray<FString>& OutImageFiles, bool bInSort = true);

	/** Get image sequence info (e.g. dimensions, number of images) from image media source asset. */
	static UE_API bool GetImageSequenceInfoFromAsset(const class UImgMediaSource* InImgSequence, FIntVector2& OutDimensions, int32& OutNumImages);

	/** Get image sequence info (e.g. dimensions, number of images) from image sequence path. */
	static UE_API bool GetImageSequenceInfoFromPath(const FString& InFullSequencePath, FIntVector2& OutDimensions, int32& OutNumImages);

	/** Returns if the image sequence is valid */
	static UE_API bool IsImageSequenceValid(const class UImgMediaSource* InImgSequence);
};

#undef UE_API
