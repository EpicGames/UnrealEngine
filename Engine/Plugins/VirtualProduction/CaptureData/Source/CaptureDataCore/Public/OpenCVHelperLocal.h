// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#define UE_API CAPTUREDATACORE_API

// NB this is a cut-down version of the FOpenCVHelper class from OpenCV runtime plugin
class FOpenCVHelperLocal
{
public:
	/** Enumeration to specify any cartesian axis in positive or negative directions */
	enum class EAxis
	{
		X, Y, Z,
		Xn, Yn, Zn,
	};

	// These axes must match the order in which they are declared in EAxis
	inline static const TArray<FVector> UnitVectors =
	{
		{  1,  0,  0 }, //  X
		{  0,  1,  0 }, //  Y
		{  0,  0,  1 }, //  Z
		{ -1,  0,  0 }, // -X
		{  0, -1,  0 }, // -Y
		{  0,  0, -1 }, // -Z
	};

	static const FVector& UnitVectorFromAxisEnum(const EAxis Axis)
	{
		return UnitVectors[std::underlying_type_t<EAxis>(Axis)];
	};

	/** Converts in-place the coordinate system of the given FTransform by specifying the source axes in terms of the destination axes */
	static UE_API void ConvertCoordinateSystem(FTransform& Transform, const EAxis DstXInSrcAxis, const EAxis DstYInSrcAxis, const EAxis DstZInSrcAxis);

	/** Converts in-place an FTransform in Unreal coordinates to OpenCV coordinates */
	static UE_API void ConvertUnrealToOpenCV(FTransform& Transform);

	/** Converts in-place an FTransform in OpenCV coordinates to Unreal coordinates */
	static UE_API void ConvertOpenCVToUnreal(FTransform& Transform);

	/** Converts an FVector in Unreal coordinates to OpenCV coordinates */
	static UE_API FVector ConvertUnrealToOpenCV(const FVector& Transform);

	/** Converts an FTransform in OpenCV coordinates to Unreal coordinates */
	static UE_API FVector ConvertOpenCVToUnreal(const FVector& Transform);

	/** Converts a FMatrix and FVector in OpenCV coordinates to FRotator and FVector in Unreal coordinates */
	static UE_API void ConvertOpenCVToUnreal(const FMatrix& InRotationOpenCV, const FVector& InTranslationOpenCV, FRotator& OutRotatorUE, FVector& OutTranslationUE);

};

#undef UE_API
