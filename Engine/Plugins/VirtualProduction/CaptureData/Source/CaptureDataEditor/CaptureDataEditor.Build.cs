// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class CaptureDataEditor : ModuleRules
{
	public CaptureDataEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
		{
			
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"AssetDefinition",
			"UnrealEd",
			"CaptureDataCore",
			"PropertyEditor",
			"SlateCore",
			"Slate",
			"InputCore",
			"Engine",
			"EditorScriptingUtilities"
		});
	}
}
