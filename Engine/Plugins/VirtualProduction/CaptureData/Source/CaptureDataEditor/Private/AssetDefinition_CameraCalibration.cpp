// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_CameraCalibration.h"
#include "CameraCalibration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CameraCalibration)

FText UAssetDefinition_CameraCalibration::GetAssetDisplayName() const
{
	return NSLOCTEXT("CaptureData", "CameraCalibrationAssetName", "Camera Calibration");
}

FLinearColor UAssetDefinition_CameraCalibration::GetAssetColor() const
{
	return FColor::Turquoise;
}

TSoftClassPtr<UObject> UAssetDefinition_CameraCalibration::GetAssetClass() const
{
	return UCameraCalibration::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CameraCalibration::GetAssetCategories() const
{
	static FAssetCategoryPath Path(NSLOCTEXT("CaptureData", "CaptureDataAssetCategoryLabel", "MetaHuman"), NSLOCTEXT("CaptureData", "CaptureDataAdvancedAssetCategoryLabel", "Advanced"), ECategoryMenuType::Section);
	static FAssetCategoryPath Categories[] = { Path };

	return Categories;
}

bool UAssetDefinition_CameraCalibration::CanImport() const
{
	return true;
}
