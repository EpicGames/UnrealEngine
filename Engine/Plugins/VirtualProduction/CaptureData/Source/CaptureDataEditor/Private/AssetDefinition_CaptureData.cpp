// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_CaptureData.h"
#include "CaptureData.h"
#include "ImageSequencePathChecker.h"

//////////////////////////////////////////////////////////////////////////
// UAssetDefinition_MeshCaptureData

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CaptureData)

FText UAssetDefinition_MeshCaptureData::GetAssetDisplayName() const
{
	return NSLOCTEXT("CaptureData", "MeshCaptureDataAssetName", "Capture Data (Mesh)");
}

FLinearColor UAssetDefinition_MeshCaptureData::GetAssetColor() const
{
	return FColor::Red;
}

TSoftClassPtr<UObject> UAssetDefinition_MeshCaptureData::GetAssetClass() const
{
	return UMeshCaptureData::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MeshCaptureData::GetAssetCategories() const
{
	static FAssetCategoryPath Path(NSLOCTEXT("CaptureData", "CaptureDataAssetCategoryLabel", "MetaHuman"));
	static FAssetCategoryPath Categories[] = { Path };

	return Categories;
}

//////////////////////////////////////////////////////////////////////////
// UAssetDefinition_FootageCaptureData

FText UAssetDefinition_FootageCaptureData::GetAssetDisplayName() const
{
	return NSLOCTEXT("CaptureData", "FootageCaptureDataAssetName", "Capture Data (Footage)");
}

FLinearColor UAssetDefinition_FootageCaptureData::GetAssetColor() const
{
	return FColor::Red;
}

TSoftClassPtr<UObject> UAssetDefinition_FootageCaptureData::GetAssetClass() const
{
	return UFootageCaptureData::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_FootageCaptureData::GetAssetCategories() const
{
	static FAssetCategoryPath Path(NSLOCTEXT("CaptureData", "CaptureDataAssetCategoryLabel", "MetaHuman"));
	static FAssetCategoryPath Categories[] = { Path };

	return Categories;
}

EAssetCommandResult UAssetDefinition_FootageCaptureData::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	UE::CaptureData::FImageSequencePathChecker ImageSequencePathChecker(GetAssetDisplayName());

	for (const UFootageCaptureData* FootageCaptureData : InOpenArgs.LoadObjects<UFootageCaptureData>())
	{
		if (FootageCaptureData)
		{
			ImageSequencePathChecker.Check(*FootageCaptureData);
		}
	}

	if (ImageSequencePathChecker.HasError())
	{
		ImageSequencePathChecker.DisplayDialog();
	}

	return UAssetDefinitionDefault::OpenAssets(InOpenArgs);
}
