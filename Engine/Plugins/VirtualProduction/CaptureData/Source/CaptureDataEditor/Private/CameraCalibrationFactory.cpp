// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationFactory.h"
#include "CameraCalibration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraCalibrationFactory)


/* UCameraCalibrationFactory
 *******************************/

UCameraCalibrationFactory::UCameraCalibrationFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCameraCalibration::StaticClass();
}


UObject* UCameraCalibrationFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	return NewObject<UCameraCalibration>(InParent, InClass, InName, InFlags);
};
