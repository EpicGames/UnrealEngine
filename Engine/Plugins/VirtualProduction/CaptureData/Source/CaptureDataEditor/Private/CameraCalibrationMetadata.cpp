// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationMetadata.h"

#include "Metadata/MetadataHandler.h"

void UCameraCalibrationMetadata::SetCameraCalibrationMetadata(UObject* InObject, const UCameraCalibrationMetadata* InCameraCalibrationMetadata)
{
	// Casting to non-const as API requests
	UE::SetMetadataObject<UCameraCalibrationMetadata>(InObject, const_cast<UCameraCalibrationMetadata*>(InCameraCalibrationMetadata));
}

UCameraCalibrationMetadata* UCameraCalibrationMetadata::GetCameraCalibrationMetadata(const UObject* InObject)
{
	return UE::GetMetadataObject<UCameraCalibrationMetadata>(InObject);
}

void UCameraCalibrationMetadata::ClearCameraCalibrationMetadata(const UObject* InObject)
{
	UE::ClearMetadataObject<UCameraCalibrationMetadata>(InObject);
}

bool UCameraCalibrationMetadata::ShowCameraCalibrationMetadataObjects(const FText& InTitle, const TArray<UObject*>& InObjects)
{
	return UE::ShowMetadataObjects(InTitle, InObjects);
}