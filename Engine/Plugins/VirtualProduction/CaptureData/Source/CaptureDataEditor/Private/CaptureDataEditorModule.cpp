// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureDataEditorModule.h"

#include "DeferredObjectDirtier.h"

#include "PropertyEditorModule.h"

#include "CaptureMetadata.h"
#include "Customizations/CaptureMetadataCustomization.h"

#include "Modules/ModuleManager.h"

void FCaptureDataEditorModule::StartupModule()
{
	CaptureMetadataName = UCaptureMetadata::StaticClass()->GetFName();
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomClassLayout(CaptureMetadataName,
												   FOnGetDetailCustomizationInstance::CreateStatic(&FCaptureMetadataCustomization::MakeInstance));
}

void FCaptureDataEditorModule::ShutdownModule()
{
	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyEditorModule->UnregisterCustomClassLayout(CaptureMetadataName);
	}
}

void FCaptureDataEditorModule::DeferMarkDirty(TWeakObjectPtr<UObject> InObject)
{
	using namespace UE::CaptureManager;
	
	FDeferredObjectDirtier& ObjectDirtier = FDeferredObjectDirtier::Get();
	ObjectDirtier.Enqueue(MoveTemp(InObject));
}

IMPLEMENT_MODULE(FCaptureDataEditorModule, CaptureDataEditor)