// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "CaptureDataEditorBridge.h"

class FCaptureDataEditorModule : public UE::CaptureManager::ICaptureDataEditorBridge
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	// ~Begin ICaptureDataEditorBridge interface
	virtual void DeferMarkDirty(TWeakObjectPtr<UObject> InObject) override;
	// ~End ICaptureDataEditorBridge interface

private:
	FName CaptureMetadataName;
};
