// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureDataFactory.h"
#include "CaptureData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CaptureDataFactory)

//////////////////////////////////////////////////////////////////////////
// UMeshCaptureDataFactory

UMeshCaptureDataFactory::UMeshCaptureDataFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UMeshCaptureData::StaticClass();
}

UObject* UMeshCaptureDataFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	return NewObject<UMeshCaptureData>(InParent, InClass, InName, InFlags);
}

//////////////////////////////////////////////////////////////////////////
// UFootageCaptureDataFactory

UFootageCaptureDataFactory::UFootageCaptureDataFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UFootageCaptureData::StaticClass();
}

UObject* UFootageCaptureDataFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	return NewObject<UFootageCaptureData>(InParent, InClass, InName, InFlags);
}
