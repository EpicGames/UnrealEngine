// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureMetadata.h"

#include "Metadata/MetadataHandler.h"

#include "UObject/Package.h"

void UCaptureMetadata::SetCaptureMetadata(UObject* InObject, const UCaptureMetadata* InCaptureMetadata)
{
	// Casting to non-const as API requests
	UE::SetMetadataObject<UCaptureMetadata>(InObject, const_cast<UCaptureMetadata*>(InCaptureMetadata));
}

UCaptureMetadata* UCaptureMetadata::GetCaptureMetadata(const UObject* InObject)
{
	return UE::GetMetadataObject<UCaptureMetadata>(InObject);
}

void UCaptureMetadata::ClearCaptureMetadata(const UObject* InObject)
{
	UE::ClearMetadataObject<UCaptureMetadata>(InObject);
}

bool UCaptureMetadata::ShowCaptureMetadataObjects(const FText& InTitle, const TArray<UObject*>& InObjects, const FCaptureMetadataWindowOptions& InOptions)
{
	for (UObject* Object : InObjects)
	{
		if (UCaptureMetadata* CaptureMetadata = Cast<UCaptureMetadata>(Object))
		{
			CaptureMetadata->bIsEditable = InOptions.bAllowEdit;
		}
	}

	return UE::ShowMetadataObjects(InTitle, InObjects);
}

bool UCaptureMetadata::IsEditable() const
{
	return bIsEditable;
}

FString UCaptureMetadata::GetOwnerName() const
{
	return OwnerName.ToString();
}
