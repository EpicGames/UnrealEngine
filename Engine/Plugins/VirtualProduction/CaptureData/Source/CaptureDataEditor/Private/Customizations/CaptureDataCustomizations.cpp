// Copyright Epic Games, Inc.All Rights Reserved.

#include "Customizations/CaptureDataCustomizations.h"
#include "CaptureData.h"
#include "FrameRangeArrayBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

TSharedRef<IDetailCustomization> FFootageCaptureDataCustomization::MakeInstance()
{
	return MakeShared<FFootageCaptureDataCustomization>();
}

void FFootageCaptureDataCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	InDetailBuilder.AddPropertyToCategory(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFootageCaptureData, ImageSequences)));
	InDetailBuilder.AddPropertyToCategory(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFootageCaptureData, DepthSequences)));
	InDetailBuilder.AddPropertyToCategory(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFootageCaptureData, AudioTracks)));
	InDetailBuilder.AddPropertyToCategory(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFootageCaptureData, CameraCalibrations)));
	InDetailBuilder.AddPropertyToCategory(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFootageCaptureData, Metadata)));

	UFootageCaptureData* FootageCaptureData = nullptr;

	// Get the performance object that we're building the details panel for.
	if (!InDetailBuilder.GetSelectedObjects().IsEmpty())
	{
		FootageCaptureData = Cast<UFootageCaptureData>(InDetailBuilder.GetSelectedObjects()[0].Get());
	}
	else
	{
		return;
	}

	TSharedRef<IPropertyHandle> CaptureExcludedFramesProperty = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFootageCaptureData, CaptureExcludedFrames));
	IDetailCategoryBuilder& CaptureExcludedFramesCategory = InDetailBuilder.EditCategory(CaptureExcludedFramesProperty->GetDefaultCategoryName());
	CaptureExcludedFramesCategory.AddCustomBuilder(MakeShareable(new FFrameRangeArrayBuilder(CaptureExcludedFramesProperty, FootageCaptureData->CaptureExcludedFrames)), false);
}
