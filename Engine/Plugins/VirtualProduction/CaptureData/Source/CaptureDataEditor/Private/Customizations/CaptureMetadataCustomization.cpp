// Copyright Epic Games, Inc.All Rights Reserved.

#include "Customizations/CaptureMetadataCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

#include "CaptureMetadata.h"

TSharedRef<IDetailCustomization> FCaptureMetadataCustomization::MakeInstance()
{
	return MakeShared<FCaptureMetadataCustomization>();
}

void FCaptureMetadataCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	InDetailBuilder.GetObjectsBeingCustomized(Objects);

	check(!Objects.IsEmpty());

	TWeakObjectPtr<UCaptureMetadata> CaptureMetadataPtr = StaticCast<UCaptureMetadata*>(Objects[0].Get());

	// We are setting the Category name for all properties to group them under a owner object name
	// 
	// When showing more then one Capture Metadata, Category Name can be used to search for the owner object
	const FString NewCategoryName = CaptureMetadataPtr->OwnerName.ToString();
	TArray<FName> CategoryNames;
	InDetailBuilder.GetCategoryNames(CategoryNames);

	for (const FName& Category : CategoryNames)
	{
		IDetailCategoryBuilder& CategoryBuilder = InDetailBuilder.EditCategory(Category);
		CategoryBuilder.SetDisplayName(FText::FromString(NewCategoryName));
	}

	InDetailBuilder.HideProperty(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBaseCaptureMetadata, OwnerName), UBaseCaptureMetadata::StaticClass()));

	if (!CaptureMetadataPtr->IsEditable())
	{
		TFieldIterator<FProperty> Iterator(InDetailBuilder.GetBaseClass());
		while (Iterator)
		{
			FProperty* Property = *Iterator;

			TSharedRef<IPropertyHandle> PropertyHandle = InDetailBuilder.GetProperty(Property->GetFName());
			IDetailPropertyRow* PropertyRow = InDetailBuilder.EditDefaultProperty(PropertyHandle);

			if (PropertyRow)
			{
				PropertyRow->IsEnabled(false);
			}

			++Iterator;
		}
	}
}