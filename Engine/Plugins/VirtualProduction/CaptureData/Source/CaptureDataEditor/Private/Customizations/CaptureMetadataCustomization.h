// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "IDetailCustomization.h"

class FCaptureMetadataCustomization
	: public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~End IDetailCustomization interface
};
