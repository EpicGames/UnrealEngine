// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeferredObjectDirtier.h"

#include "UObject/Object.h"

namespace UE::CaptureManager
{

FDeferredObjectDirtier& FDeferredObjectDirtier::Get()
{
	static FDeferredObjectDirtier Instance;
	return Instance;
}

void FDeferredObjectDirtier::Enqueue(TWeakObjectPtr<UObject> InObject)
{
	check(IsInGameThread());
	ObjectsToMarkDirty.Emplace(MoveTemp(InObject));
}

void FDeferredObjectDirtier::Tick(const float InDeltaTime)
{
	for (TWeakObjectPtr<UObject>& ObjectToMarkDirty : ObjectsToMarkDirty)
	{
		if (ObjectToMarkDirty.IsValid())
		{
			ObjectToMarkDirty->Modify();
			ObjectToMarkDirty->MarkPackageDirty();
		}
	}

	ObjectsToMarkDirty.Empty();
}

bool FDeferredObjectDirtier::IsTickable() const
{
	return !ObjectsToMarkDirty.IsEmpty();
}

TStatId FDeferredObjectDirtier::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDeferredObjectDirtier, STATGROUP_Tickables);
}

} // namespace UE::CaptureManager

