// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TickableEditorObject.h"

class UObject;

namespace UE::CaptureManager
{

class FDeferredObjectDirtier : public FTickableEditorObject
{
public:
	static FDeferredObjectDirtier& Get();
	void Enqueue(TWeakObjectPtr<UObject> InObject);

	virtual void Tick(float InDeltaTime) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;

private:
	FDeferredObjectDirtier() = default;
	TArray<TWeakObjectPtr<UObject>> ObjectsToMarkDirty;
};

} // namespace UE::CaptureManager

