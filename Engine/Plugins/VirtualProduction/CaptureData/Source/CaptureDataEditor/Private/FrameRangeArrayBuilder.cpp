// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameRangeArrayBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SSpinBox.h"
#include "Fonts/FontMeasure.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "FrameRange"

FFrameRangeArrayBuilder::FFrameRangeArrayBuilder(TSharedRef<IPropertyHandle> InBaseProperty, TArray<FFrameRange>& InOutFrameRange, FOnGetCurrentFrame* InOnGetCurrentFrameDelegate)
	: FDetailArrayBuilder(InBaseProperty)
	, FrameRange(InOutFrameRange)
	, OnGetCurrentFrameDelegate(InOnGetCurrentFrameDelegate)
{
}

void FFrameRangeArrayBuilder::GenerateChildContent(IDetailChildrenBuilder& InOutChildrenBuilder)
{
	const FVector2f LabelSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(TEXT("Start"), IDetailLayoutBuilder::GetDetailFont()) * 1.5;

	for (int32 Index = 0; Index < FrameRange.Num(); ++Index)
	{
		TSharedRef<SHorizontalBox> StartRow = SNew(SHorizontalBox);

		StartRow->AddSlot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.MinDesiredWidth(LabelSize.X)
				.Text(LOCTEXT("Start", "Start"))
			];

		StartRow->AddSlot()
			[
				SNew(SSpinBox<int32>)
				.IsEnabled_Lambda([this]()
				{
					return GetPropertyHandle()->IsEditable();
				})
				.Value_Lambda([this, Index]()
				{
					return Index < FrameRange.Num() ? FrameRange[Index].StartFrame : -2; // Index can be temporarily invalid when deleting an entry, return an arbitrary value - should never be actually visible
				})
				.OnValueChanged_Lambda([this, Index](int32 InValue)
				{
					const FScopedTransaction Transaction{ LOCTEXT("SetStartFrame", "Set Start Frame") };

					GetPropertyHandle()->NotifyPreChange();
					if (FrameRange[Index].StartFrame == -1 && FrameRange[Index].EndFrame == -1)
					{
						FrameRange[Index].EndFrame = InValue;
					}
					FrameRange[Index].StartFrame = InValue;
					GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);
				})
			];

		if (OnGetCurrentFrameDelegate)
		{
			StartRow->AddSlot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("Set", "Set"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this, Index]()
					{
						if (OnGetCurrentFrameDelegate->IsBound())
						{
							const FScopedTransaction Transaction{ LOCTEXT("SetStartFrame", "Set Start Frame") };

							GetPropertyHandle()->NotifyPreChange();
							if (FrameRange[Index].StartFrame == -1 && FrameRange[Index].EndFrame == -1)
							{
								FrameRange[Index].EndFrame = OnGetCurrentFrameDelegate->Execute().Value;
							}
							FrameRange[Index].StartFrame = OnGetCurrentFrameDelegate->Execute().Value;
							GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);
						}

						return FReply::Handled();
					})
				];
		}


		TSharedRef<SHorizontalBox> EndRow = SNew(SHorizontalBox);

		EndRow->AddSlot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.MinDesiredWidth(LabelSize.X)
				.Text(LOCTEXT("End", "End"))
			];

		EndRow->AddSlot()
			[
				SNew(SSpinBox<int32>)
				.IsEnabled_Lambda([this]()
				{
					return GetPropertyHandle()->IsEditable();
				})
				.Value_Lambda([this, Index]()
				{
					return Index < FrameRange.Num() ? FrameRange[Index].EndFrame : -2; // Index can be temporarily invalid when deleting an entry, return an arbitrary value - should never be actually visible
				})
				.OnValueChanged_Lambda([this, Index](int32 InValue)
				{
					const FScopedTransaction Transaction{ LOCTEXT("SetEndFrame", "Set End Frame") };

					GetPropertyHandle()->NotifyPreChange();
					if (FrameRange[Index].StartFrame == -1 && FrameRange[Index].EndFrame == -1)
					{
						FrameRange[Index].StartFrame = InValue;
					}
					FrameRange[Index].EndFrame = InValue;
					GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);
				})
			];

		if (OnGetCurrentFrameDelegate)
		{
			EndRow->AddSlot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("Set", "Set"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this, Index]()
					{
						if (OnGetCurrentFrameDelegate->IsBound())
						{
							const FScopedTransaction Transaction{ LOCTEXT("SetEndFrame", "Set End Frame") };

							GetPropertyHandle()->NotifyPreChange();
							if (FrameRange[Index].StartFrame == -1 && FrameRange[Index].EndFrame == -1)
							{
								FrameRange[Index].StartFrame = OnGetCurrentFrameDelegate->Execute().Value;
							}
							FrameRange[Index].EndFrame = OnGetCurrentFrameDelegate->Execute().Value;
							GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);
						}

						return FReply::Handled();
					})
				];
		}


		TSharedRef<SVerticalBox> FrameControls = SNew(SVerticalBox);

		FrameControls->AddSlot()
			[
				StartRow
			];

		FrameControls->AddSlot()
			[
				EndRow
			];

		if (GetPropertyHandle()->IsEditable())
		{
			FrameControls->AddSlot()
				[
					SNew(SButton)
					.Text(LOCTEXT("Delete", "Delete"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this, Index]()
					{
						const FScopedTransaction Transaction{ LOCTEXT("DeleteFrameRange", "Delete Frame Range") };

						GetPropertyHandle()->NotifyPreChange();
						FrameRange.RemoveAt(Index);
						GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ArrayRemove);

						return FReply::Handled();
					})
				];
		}


		InOutChildrenBuilder.AddCustomRow(LOCTEXT("Frame", "Frame"))
			.NameContent()
			[
				SNew(SEditableTextBox)
				.Text_Lambda([this, Index]()
				{
					return Index < FrameRange.Num() ? FText::FromString(FrameRange[Index].Name) : FText(); // Index can be temporarily invalid when deleting an entry, return an arbitrary value - should never be actually visible
				})
				.OnTextCommitted_Lambda([this, Index](FText InValue, ETextCommit::Type)
				{
					const FScopedTransaction Transaction{ LOCTEXT("NameFrameRange", "Name Frame Range") };

					GetPropertyHandle()->NotifyPreChange();
					FrameRange[Index].Name = InValue.ToString();
					GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);						
				})
			]
			.ValueContent()
			.MinDesiredWidth(450.0f)
			.MaxDesiredWidth(0.0f)
			[
				FrameControls
			];
	}
}

#undef LOCTEXT_NAMESPACE
