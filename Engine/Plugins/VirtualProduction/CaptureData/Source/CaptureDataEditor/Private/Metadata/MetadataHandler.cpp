// Copyright Epic Games, Inc. All Rights Reserved.

#include "Metadata/MetadataHandler.h"

#if WITH_EDITOR
#include "EditorDialogLibrary.h"
#endif

namespace UE
{

bool ShowMetadataObjects(const FText& InTitle, const TArray<UObject*>& InObjects)
{
#if WITH_EDITOR
	FEditorDialogLibraryObjectDetailsViewOptions Options;
	Options.bShowObjectName = false;
	Options.bAllowResizing = true;
	Options.MinWidth = 400;
	Options.MinHeight = 200;

	return UEditorDialogLibrary::ShowObjectsDetailsView(InTitle, InObjects, Options);
#else
	return false;
#endif
}

}
