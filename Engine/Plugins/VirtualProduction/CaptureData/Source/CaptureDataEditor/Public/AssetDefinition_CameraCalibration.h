// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_CameraCalibration.generated.h"

UCLASS()
class UAssetDefinition_CameraCalibration
	: public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~Begin UAssetDefinitionDefault interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override;
	//~End UAssetDefinitionDefault interface
};