// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_CaptureData.generated.h"

//////////////////////////////////////////////////////////////////////////
// UAssetDefinition_MeshCaptureData

UCLASS()
class UAssetDefinition_MeshCaptureData
	: public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~Begin UAssetDefinitionDefault interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	//~End UAssetDefinitionDefault interface
};

//////////////////////////////////////////////////////////////////////////
// UAssetDefinition_FootageCaptureData

UCLASS()
class UAssetDefinition_FootageCaptureData
	: public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~Begin UAssetDefinitionDefault interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;

	//~End UAssetDefinitionDefault interface
};