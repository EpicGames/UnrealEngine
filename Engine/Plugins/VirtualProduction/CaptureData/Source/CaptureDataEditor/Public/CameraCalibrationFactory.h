// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CameraCalibrationFactory.generated.h"

/**
 * Implements a factory for UCameraCalibrationFactory objects from file
 */
UCLASS(hidecategories=Object)
class UCameraCalibrationFactory
	: public UFactory
{
	GENERATED_BODY()

public:
	UCameraCalibrationFactory();

	//~ UFactory interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
};
