// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Metadata/MetadataHandler.h"

#include "Misc/Timecode.h"

#include "CameraCalibrationMetadata.generated.h"

#define UE_API CAPTUREDATAEDITOR_API

UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UCameraCalibrationMetadata : public UBaseCaptureMetadata
{
public:

	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category = "Camera Calibration Metadata")
	static UE_API void SetCameraCalibrationMetadata(UObject* InObject, const UCameraCalibrationMetadata* InCameraCalibrationMetadata);

	UFUNCTION(BlueprintCallable, Category = "Camera Calibration Metadata")
	static UE_API UCameraCalibrationMetadata* GetCameraCalibrationMetadata(const UObject* InObject);

	UFUNCTION(BlueprintCallable, Category = "Camera Calibration Metadata")
	static UE_API void ClearCameraCalibrationMetadata(const UObject* InObject);

	UFUNCTION(BlueprintCallable, Category = "Camera Calibration Metadata")
	static UE_API bool ShowCameraCalibrationMetadataObjects(const FText& InTitle, const TArray<UObject*>& InObjects);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn), Category = "Camera Calibration Metadata")
	double ReprojectionRMSError = 0.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn), Category = "Camera Calibration Metadata")
	FTimecode GenerationTimecode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn), Category = "Camera Calibration Metadata")
	FFrameRate GenerationFrameRate;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn), Category = "Camera Calibration Metadata")
	TArray<int32> SelectedFrames;
};

#undef UE_API