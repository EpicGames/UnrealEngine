// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CaptureDataFactory.generated.h"


//////////////////////////////////////////////////////////////////////////
// UMeshCaptureDataFactory

UCLASS(hidecategories=Object)
class UMeshCaptureDataFactory
	: public UFactory
{
	GENERATED_BODY()

public:
	//~ UFactory Interface
	UMeshCaptureDataFactory();

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
};

//////////////////////////////////////////////////////////////////////////
// UFootageCaptureDataFactory

UCLASS(hideCategories=Object)
class UFootageCaptureDataFactory
	: public UFactory
{
	GENERATED_BODY()

public:
	//~ UFactory Interface
	UFootageCaptureDataFactory();

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
};