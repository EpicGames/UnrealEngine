// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Metadata/MetadataHandler.h"

#include "CaptureMetadata.generated.h"

#define UE_API CAPTUREDATAEDITOR_API

USTRUCT(MinimalAPI, BlueprintType, Blueprintable)
struct FCaptureMetadataWindowOptions
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture Metadata | Options")
	bool bAllowEdit = true;
};

UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UCaptureMetadata
	: public UBaseCaptureMetadata
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Capture Metadata")
	static UE_API void SetCaptureMetadata(UObject* InObject, const UCaptureMetadata* InCaptureMetadata);

	UFUNCTION(BlueprintCallable, Category = "Capture Metadata")
	static UE_API UCaptureMetadata* GetCaptureMetadata(const UObject* InObject);

	UFUNCTION(BlueprintCallable, Category = "Capture Metadata")
	static UE_API void ClearCaptureMetadata(const UObject* InObject);

	UFUNCTION(BlueprintCallable, Category = "Capture Metadata")
	static UE_API bool ShowCaptureMetadataObjects(const FText& InTitle, const TArray<UObject*>& InObjects, const FCaptureMetadataWindowOptions& InOptions);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn), Category = "Capture Metadata")
	FString CameraId;

	bool IsEditable() const;
	FString GetOwnerName() const;

private:

	bool bIsEditable = true;
};

#undef UE_API