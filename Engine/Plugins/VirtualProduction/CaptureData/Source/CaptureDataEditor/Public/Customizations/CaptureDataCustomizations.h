// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "IDetailCustomization.h"

#define UE_API CAPTUREDATAEDITOR_API

class FFootageCaptureDataCustomization
	: public IDetailCustomization
{
public:
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();

	//~Begin IDetailCustomization interface
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~End IDetailCustomization interface
};

#undef UE_API
