// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrameRange.h"
#include "PropertyCustomizationHelpers.h"

#define UE_API CAPTUREDATAEDITOR_API

class FFrameRangeArrayBuilder : public FDetailArrayBuilder
{
public:

	DECLARE_DELEGATE_RetVal(FFrameNumber, FOnGetCurrentFrame);

	UE_API FFrameRangeArrayBuilder(TSharedRef<IPropertyHandle> InBaseProperty, TArray<FFrameRange>& InOutFrameRange, FOnGetCurrentFrame* InOnGetCurrentFrameDelegate = nullptr);

	UE_API virtual void GenerateChildContent(IDetailChildrenBuilder& InOutChildrenBuilder) override;

private:

	TArray<FFrameRange>& FrameRange;
	FOnGetCurrentFrame* OnGetCurrentFrameDelegate = nullptr;
};

#undef UE_API
