// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/Package.h"

#include "Concepts/DerivedFrom.h"

#include "MetadataHandler.generated.h"

UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UBaseCaptureMetadata : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Base Capture Metadata")
	FName OwnerName;
};

namespace UE
{

template<typename T>
	requires UE::CDerivedFrom<T, UBaseCaptureMetadata>
void SetMetadataObject(UObject* InObject, T* InMetadata)
{
	if (!IsValid(InObject))
	{
		return;
	}

	if (!IsValid(InMetadata))
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	FMetaData& Metadata = InObject->GetPackage()->GetMetaData();

	TFieldIterator<FProperty> It(InMetadata->GetClass());

	bool bIsEdited = false;

	while (It)
	{
		FProperty* Property = *It;

		if (Property->GetFName() != GET_MEMBER_NAME_CHECKED(UBaseCaptureMetadata, OwnerName))
		{
			const void* ValueAddr = Property->ContainerPtrToValuePtr<void>(InMetadata);

			FString Value;
			Property->ExportText_Direct(Value, ValueAddr, ValueAddr, InMetadata, 0);

			FString OldValue;
			if (Metadata.HasValue(InObject, Property->GetFName()))
			{
				OldValue = Metadata.GetValue(InObject, Property->GetFName());
			}

			if (Value != OldValue)
			{
				Metadata.SetValue(InObject, Property->GetFName(), *Value);

				bIsEdited = true;
			}
		}

		++It;
	}

	if (bIsEdited)
	{
		InObject->MarkPackageDirty();
	}
#endif
}

template<typename T>
	requires UE::CDerivedFrom<T, UBaseCaptureMetadata>
T* GetMetadataObject(const UObject* InObject)
{
	if (!IsValid(InObject))
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	FMetaData& Metadata = InObject->GetPackage()->GetMetaData();

	TMap<FName, FString>* MetadataMap = Metadata.GetMapForObject(InObject);
	if (!MetadataMap)
	{
		return nullptr;
	}

	T* MetadataObject = NewObject<T>(GetTransientPackage());
	MetadataObject->OwnerName = InObject->GetFName();

	for (const TPair<FName, FString>& MetadataPair : *MetadataMap)
	{
		if (MetadataPair.Key == GET_MEMBER_NAME_CHECKED(UBaseCaptureMetadata, OwnerName))
		{
			continue;
		}

		FProperty* Property = MetadataObject->GetClass()->FindPropertyByName(MetadataPair.Key);

		if (Property)
		{
			void* ValueAddr = Property->ContainerPtrToValuePtr<void>(MetadataObject);
			Property->ImportText_Direct(*MetadataPair.Value, ValueAddr, MetadataObject, 0);
		}
	}

	return MetadataObject;
#else
	return nullptr;
#endif
}

template<typename T>
	requires UE::CDerivedFrom<T, UBaseCaptureMetadata>
void ClearMetadataObject(const UObject* InObject)
{
	if (!IsValid(InObject))
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	FMetaData& Metadata = InObject->GetPackage()->GetMetaData();

	TMap<FName, FString>* MetadataMap = Metadata.GetMapForObject(InObject);
	if (!MetadataMap)
	{
		return;
	}

	TMap<FName, FString> MetadataCopy = *MetadataMap;

	bool bIsEdited = false;
	for (const TPair<FName, FString>& MetadataPair : MetadataCopy)
	{
		FProperty* Property = T::StaticClass()->FindPropertyByName(MetadataPair.Key);

		if (Property)
		{
			if (Metadata.HasValue(InObject, MetadataPair.Key))
			{
				Metadata.RemoveValue(InObject, MetadataPair.Key);

				bIsEdited = true;
			}
		}
	}

	if (bIsEdited)
	{
		InObject->MarkPackageDirty();
	}
#endif
}

bool ShowMetadataObjects(const FText& InTitle, const TArray<UObject*>& InObjects);

}