// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CaptureDataUtils : ModuleRules
{
	public CaptureDataUtils(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"ImgMedia",
		});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd",
				"EditorScriptingUtilities"
			});
		}
	}
}
