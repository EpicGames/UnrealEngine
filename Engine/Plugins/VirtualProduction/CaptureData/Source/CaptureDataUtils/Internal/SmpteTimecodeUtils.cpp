// Copyright Epic Games, Inc.All Rights Reserved.

#include "SmpteTimecodeUtils.h"

namespace UE::CaptureData
{

FFrameRate EstimateSmpteTimecodeRate(const FFrameRate InMediaFrameRate)
{
	constexpr double Tolerance = UE_DOUBLE_KINDA_SMALL_NUMBER;

	if (FMath::IsNearlyEqual(InMediaFrameRate.AsDecimal(), 60.0, Tolerance))
	{
		return FFrameRate(30'000, 1'000);
	}

	if (FMath::IsNearlyEqual(InMediaFrameRate.AsDecimal(), 59.94, Tolerance))
	{
		// 29.97
		return FFrameRate(30'000, 1'001);
	}

	if (FMath::IsNearlyEqual(InMediaFrameRate.AsDecimal(), 50.0, Tolerance))
	{
		return FFrameRate(25'000, 1'000);
	}

	if (FMath::IsNearlyEqual(InMediaFrameRate.AsDecimal(), 48.0, Tolerance))
	{
		return FFrameRate(24'000, 1'000);
	}

	if (FMath::IsNearlyEqual(InMediaFrameRate.AsDecimal(), 47.952, Tolerance))
	{
		// 23.976 = 24000/1001 (canonical NTSC pulldown rate)
		return FFrameRate(24'000, 1'001);
	}

	return InMediaFrameRate;
}

}