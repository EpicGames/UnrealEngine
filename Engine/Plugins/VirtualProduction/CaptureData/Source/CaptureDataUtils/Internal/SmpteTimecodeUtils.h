// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"

namespace UE::CaptureData
{

UE_INTERNAL CAPTUREDATAUTILS_API FFrameRate EstimateSmpteTimecodeRate(const FFrameRate InMediaFrameRate);

}