// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmpteTimecodeUtils.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::MetaHuman::Private
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_5over1,
	"MetaHuman.Capture.SmpteTimecodeUtils.InputArbitraryFrameRate",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_5over1::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(5, 1);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, InputFrameRate);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_30000over1000,
	"MetaHuman.Capture.SmpteTimecodeUtils.Input_30fps_(30000/1000)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_30000over1000::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(30'000, 1'000);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, InputFrameRate);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_60over1,
	"MetaHuman.Capture.SmpteTimecodeUtils.Input_60fps_(60/1)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_60over1::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(60, 1);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, FFrameRate(30'000, 1'000));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_60000over1000,
	"MetaHuman.Capture.SmpteTimecodeUtils.Input_60fps_(60000/1000)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_60000over1000::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(60'000, 1000);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, FFrameRate(30'000, 1'000));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_60000over1001,
	"MetaHuman.Capture.SmpteTimecodeUtils.Input_59.94fps_(60000/1001)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_60000over1001::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(60'000, 1001);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, FFrameRate(30'000, 1'001));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_50over1,
	"MetaHuman.Capture.SmpteTimecodeUtils.Input_50fps_(50/1)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_50over1::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(50, 1);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, FFrameRate(25'000, 1'000));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_50000over1000,
	"MetaHuman.Capture.SmpteTimecodeUtils.Input_50fps_(50000/1000)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_50000over1000::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(50'000, 1000);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, FFrameRate(25'000, 1'000));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_48over1,
	"MetaHuman.Capture.SmpteTimecodeUtils.Input_48fps_(48/1)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_48over1::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(48, 1);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, FFrameRate(24'000, 1'000));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSmpteTimecodeUtils_48000over1000,
	"MetaHuman.Capture.SmpteTimecodeUtils.Input_48fps_(48000/1000)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter
)
bool FSmpteTimecodeUtils_48000over1000::RunTest(const FString& InParameters)
{
	FFrameRate InputFrameRate = FFrameRate(48'000, 1000);

	FFrameRate OutputFrameRate = UE::CaptureData::EstimateSmpteTimecodeRate(InputFrameRate);

	UTEST_EQUAL("Smpte Timecode Rate", OutputFrameRate, FFrameRate(24'000, 1'000));

	return true;
}

}

#endif