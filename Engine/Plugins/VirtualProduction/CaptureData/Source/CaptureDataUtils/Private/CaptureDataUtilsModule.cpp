// Copyright Epic Games, Inc.All Rights Reserved.

#include "Modules/ModuleManager.h"

using FCaptureDataUtilsModule = IModuleInterface;

IMPLEMENT_MODULE(FCaptureDataUtilsModule, CaptureDataUtils);