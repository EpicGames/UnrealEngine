// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Misc/Timecode.h"
#include "Misc/FrameRate.h"

#include "CaptureTimecodeInfo.generated.h"

UCLASS(BlueprintType, Blueprintable)
class UCaptureTimecodeInfo
	: public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn), Category = "Info")
	FTimecode Timecode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ExposeOnSpawn), Category = "Info")
	FFrameRate FrameRate;
};
