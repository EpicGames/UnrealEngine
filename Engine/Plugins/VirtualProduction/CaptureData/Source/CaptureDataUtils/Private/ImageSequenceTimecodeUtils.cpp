// Copyright Epic Games, Inc.All Rights Reserved.

#include "ImageSequenceTimecodeUtils.h"

#include "ParseTakeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImageSequenceTimecodeUtils)

void UImageSequenceTimecodeUtils::SetTimecodeInfo(const FTimecode& InTimecode, const FFrameRate& InFrameRate, UImgMediaSource* InImageSequence)
{
	if (ensure(IsValid(InImageSequence)))
	{
		InImageSequence->StartTimecode = InTimecode;
		InImageSequence->FrameRateOverride = InFrameRate;
	}
}

void UImageSequenceTimecodeUtils::SetTimecodeInfoString(const FString& InTimecode, const FString& InFrameRate, UImgMediaSource* InImageSequence)
{
	if (ensure(IsValid(InImageSequence)))
	{
		InImageSequence->StartTimecode = ParseTimecode(InTimecode);

		double TimecodeRate = FCString::Atod(*InFrameRate);
		InImageSequence->FrameRateOverride = ConvertFrameRate(TimecodeRate);
	}
}

FTimecode UImageSequenceTimecodeUtils::GetTimecode(UImgMediaSource* InImageSequence)
{
	if (ensure(IsValid(InImageSequence)))
	{
		return InImageSequence->StartTimecode;
	}
	return FTimecode();
}

FFrameRate UImageSequenceTimecodeUtils::GetFrameRate(UImgMediaSource* InImageSequence)
{
	if (ensure(IsValid(InImageSequence)))
	{
		return InImageSequence->FrameRateOverride;
	}
	return FFrameRate();
}

FString UImageSequenceTimecodeUtils::GetTimecodeString(UImgMediaSource* InImageSequence)
{
	if (ensure(IsValid(InImageSequence)))
	{
		return InImageSequence->StartTimecode.ToString();
	}
	return FString();
}

FString UImageSequenceTimecodeUtils::GetFrameRateString(UImgMediaSource* InImageSequence)
{
	if (ensure(IsValid(InImageSequence)))
	{
		return FString::SanitizeFloat(InImageSequence->FrameRateOverride.AsDecimal());
	}
	return FString();
}

bool UImageSequenceTimecodeUtils::IsValidTimecodeInfo(const FTimecode& InTimecode, const FFrameRate& InTimecodeRate)
{
	return IsValidTimecode(InTimecode) && IsValidFrameRate(InTimecodeRate);
}

bool UImageSequenceTimecodeUtils::IsValidTimecode(const FTimecode& InTimecode)
{
	return InTimecode.IsValid() && InTimecode != FTimecode();
}

bool UImageSequenceTimecodeUtils::IsValidFrameRate(const FFrameRate& InTimecodeRate)
{
	return InTimecodeRate.IsValid() && InTimecodeRate != FFrameRate() && InTimecodeRate.Numerator > 0;
}
