// Copyright Epic Games, Inc.All Rights Reserved.

#include "ParseTakeUtils.h"

#include "CoreTypes.h"
#include "Containers/Array.h"

FTimecode ParseTimecode(const FString& InTimecodeString)
{
	TArray<FString> ParsedStringArray;
	InTimecodeString.ParseIntoArray(ParsedStringArray, TEXT(":"));

	if (ParsedStringArray.Num() != 4 && ParsedStringArray.Num() != 3)
	{
		return FTimecode(0, 0, 0, 0, false);
	}

	int32 Hours = FCString::Atoi(*ParsedStringArray[0]);
	int32 Minutes = FCString::Atoi(*ParsedStringArray[1]);

	int32 Seconds = 0;
	int32 Frames = 0;
	bool bIsFrameDrop = ParsedStringArray[2].Contains(TEXT(";"));
	if (bIsFrameDrop)
	{
		// ParsedStringArray[2] == 00;00
		TArray<FString> SecondsAndFrames;
		ParsedStringArray[2].ParseIntoArray(SecondsAndFrames, TEXT(";"));

		if (SecondsAndFrames.Num() != 2)
		{
			return FTimecode(0, 0, 0, 0, false);
		}

		Seconds = FCString::Atoi(*SecondsAndFrames[0]);
		Frames = FCString::Atoi(*SecondsAndFrames[1]);
	}
	else
	{
		Seconds = FCString::Atoi(*ParsedStringArray[2]);
		Frames = FCString::Atoi(*ParsedStringArray[3]);
	}

	return FTimecode(Hours, Minutes, Seconds, Frames, bIsFrameDrop);
}

FFrameRate ConvertFrameRate(double InFrameRate)
{
	return FFrameRate(FMath::CeilToInt(InFrameRate) * 1000,
					  FMath::IsNearlyZero(FMath::Fractional(InFrameRate)) ? 1000 : 1001);
}