// Copyright Epic Games, Inc.All Rights Reserved.

#include "SoundWaveTimecodeUtils.h"

#include "Sound/SoundWave.h"
#include "Sound/SoundWaveTimecodeInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundWaveTimecodeUtils)

DEFINE_LOG_CATEGORY_STATIC(LogSoundWaveTimecodeUtils, Log, All);

void USoundWaveTimecodeUtils::SetTimecodeInfo(const FTimecode& InTimecode, const FFrameRate& InFrameRate, USoundWave* OutSoundWave)
{
	if (!ensure(IsValid(OutSoundWave)))
	{
		UE_LOGF(LogSoundWaveTimecodeUtils, Error, "[%ls] Sound wave pointer is invalid", ANSI_TO_TCHAR(__FUNCTION__));
		return;
	}

	// Sanity check that the frame rate has actually been set and is not just default constructed (60'000 fps)
	check(InFrameRate != FFrameRate());

	const float SampleRate = OutSoundWave->GetSampleRateForCurrentPlatform();
	check(SampleRate > 0.0f);

#if WITH_EDITORONLY_DATA
	FSoundWaveTimecodeInfo SoundWaveTimecodeInfo;
	SoundWaveTimecodeInfo.NumSamplesPerSecond = SampleRate;
	SoundWaveTimecodeInfo.TimecodeRate = InFrameRate;
	SoundWaveTimecodeInfo.NumSamplesSinceMidnight = InTimecode.ToTimespan(InFrameRate).GetTotalSeconds() * SoundWaveTimecodeInfo.NumSamplesPerSecond;
	SoundWaveTimecodeInfo.bTimecodeIsDropFrame = InTimecode.bDropFrameFormat;
	OutSoundWave->SetTimecodeInfo(SoundWaveTimecodeInfo);
#else 
	// This function may not really belong in this module...
	check(false);
	UE_LOGF(LogSoundWaveTimecodeUtils, Error, "Function %ls is not supported in non-editor targets", ANSI_TO_TCHAR(__FUNCTION__));
#endif
}

FTimecode USoundWaveTimecodeUtils::GetTimecode(const USoundWave* InSoundWave)
{
	if (!ensure(IsValid(InSoundWave)))
	{
		UE_LOGF(LogSoundWaveTimecodeUtils, Error, "[%ls] Sound wave pointer is invalid", ANSI_TO_TCHAR(__FUNCTION__));
		return {};
	}

	TOptional<FSoundWaveTimecodeInfo> SoundWaveTimecodeInfo;

#if WITH_EDITORONLY_DATA
	SoundWaveTimecodeInfo = InSoundWave->GetTimecodeInfo();
#else 
	// This function may not really belong in this module...
	check(false);
	UE_LOGF(LogSoundWaveTimecodeUtils, Error, "Function %ls is not supported in non-editor targets", ANSI_TO_TCHAR(__FUNCTION__));
#endif

	if (!SoundWaveTimecodeInfo)
	{
		return {};
	}

	// Sanity check that the frame rate has actually been set and is not just default constructed (60'000 fps)
	check(SoundWaveTimecodeInfo->TimecodeRate != FFrameRate());

	// GetNumSecondsSinceMidnight will return 0.0 in this case, so nothing here will fail, but it may be useful to fail early during dev
	check(SoundWaveTimecodeInfo->NumSamplesPerSecond > 0);

	constexpr bool bRollover = true;
	FTimecode Timecode(SoundWaveTimecodeInfo->GetNumSecondsSinceMidnight(),
					   SoundWaveTimecodeInfo->TimecodeRate,
					   SoundWaveTimecodeInfo->bTimecodeIsDropFrame,
					   bRollover);

	return Timecode;
}

FFrameRate USoundWaveTimecodeUtils::GetFrameRate(const USoundWave* InSoundWave)
{
	if (!ensure(IsValid(InSoundWave)))
	{
		UE_LOGF(LogSoundWaveTimecodeUtils, Error, "[%ls] Sound wave pointer is invalid", ANSI_TO_TCHAR(__FUNCTION__));
		return {};
	}

	TOptional<FSoundWaveTimecodeInfo> SoundWaveTimecodeInfo;

#if WITH_EDITORONLY_DATA
	SoundWaveTimecodeInfo = InSoundWave->GetTimecodeInfo();
#else 
	// This function may not really belong in this module...
	check(false);
	UE_LOGF(LogSoundWaveTimecodeUtils, Error, "Function %ls is not supported in non-editor targets", ANSI_TO_TCHAR(__FUNCTION__));
#endif

	if (!SoundWaveTimecodeInfo)
	{
		return {};
	}

	return SoundWaveTimecodeInfo->TimecodeRate;
}

bool USoundWaveTimecodeUtils::HasTimecode(const USoundWave* InSoundWave)
{
	if (!ensure(IsValid(InSoundWave)))
	{
		UE_LOGF(LogSoundWaveTimecodeUtils, Error, "[%ls] Sound wave pointer is invalid", ANSI_TO_TCHAR(__FUNCTION__));
		return false;
	}

	TOptional<FSoundWaveTimecodeInfo> SoundWaveTimecodeInfo;

#if WITH_EDITORONLY_DATA
	SoundWaveTimecodeInfo = InSoundWave->GetTimecodeInfo();
#else 
	// This function may not really belong in this module...
	check(false);
	UE_LOGF(LogSoundWaveTimecodeUtils, Error, "Function %ls is not supported in non-editor targets", ANSI_TO_TCHAR(__FUNCTION__));
#endif

	if (!SoundWaveTimecodeInfo)
	{
		return false;
	}

	if (!SoundWaveTimecodeInfo->TimecodeRate.IsValid() || SoundWaveTimecodeInfo->TimecodeRate.Numerator <= 0 || SoundWaveTimecodeInfo->TimecodeRate == FFrameRate())
	{
		return false;
	}

	if (SoundWaveTimecodeInfo->NumSamplesPerSecond == 0)
	{
		return false;
	}

	return true;
}

bool USoundWaveTimecodeUtils::HasTimecodeRate(const USoundWave* InSoundWave)
{
	if (!ensure(IsValid(InSoundWave)))
	{
		UE_LOGF(LogSoundWaveTimecodeUtils, Error, "[%ls] Sound wave pointer is invalid", ANSI_TO_TCHAR(__FUNCTION__));
		return false;
	}

	TOptional<FSoundWaveTimecodeInfo> SoundWaveTimecodeInfo;

#if WITH_EDITORONLY_DATA
	SoundWaveTimecodeInfo = InSoundWave->GetTimecodeInfo();
#else 
	// This function may not really belong in this module...
	check(false);
	UE_LOGF(LogSoundWaveTimecodeUtils, Error, "Function %ls is not supported in non-editor targets", ANSI_TO_TCHAR(__FUNCTION__));
#endif

	if (!SoundWaveTimecodeInfo)
	{
		return false;
	}

	if (!SoundWaveTimecodeInfo->TimecodeRate.IsValid() || SoundWaveTimecodeInfo->TimecodeRate.Numerator <= 0 || SoundWaveTimecodeInfo->TimecodeRate == FFrameRate())
	{
		return false;
	}

	return true;
}
