// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeDirectoryUtils.h"

#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "IImageWrapperModule.h"

#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

namespace UE::CaptureData
{

FRegexPattern GetRegexPattern()
{
	return FRegexPattern(TEXT("^(.*?)(\\d+)\\.(\\w+)$"));
}

bool ExtractInfoFromFileName(const FRegexPattern& InPattern, const FString& InFileName, FString& OutPrefix, FString& OutDigits, FString& OutExtension)
{
	FRegexMatcher Matcher(InPattern, InFileName);

	if (Matcher.FindNext())
	{
		OutPrefix = Matcher.GetCaptureGroup(1);
		OutDigits = Matcher.GetCaptureGroup(2);
		OutExtension = Matcher.GetCaptureGroup(3);

		return true;
	}

	return false;
}

FString GetFileFormat(const FRegexPattern& InPattern, const FString& InFileName)
{
	FString Prefix;
	FString Digits;
	FString Extension;
	if (ExtractInfoFromFileName(InPattern, InFileName, Prefix, Digits, Extension))
	{
		return FString::Format(TEXT("{0}%0{1}d.{2}"), { Prefix, FString::FromInt(Digits.Len()), Extension });
	}

	return FString();
}

FString GetFileNameFormat(const FString& InDirectory)
{
	IFileManager& FileManager = IFileManager::Get();
	IImageWrapperModule& ImageWrapperModule =
		FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	FString FileName;
	FileManager.IterateDirectory(*InDirectory, [&FileName, &ImageWrapperModule](const TCHAR* InFilenameOrDirectory, bool bInIsDirectory)
								 {
									 if (bInIsDirectory)
									 {
										 return true;
									 }

									 FString Extension = FPaths::GetExtension(InFilenameOrDirectory, false);
									 if (ImageWrapperModule.GetImageFormatFromExtension(*Extension) != EImageFormat::Invalid)
									 {
										 // Found the correct extension and return false to exit the iteration
										 FileName = FPaths::GetCleanFilename(InFilenameOrDirectory);

										 return false;
									 }

									 return true;
								 });

	return GetFileFormat(GetRegexPattern(), FileName);
}

TArray<FString> GetImageSequenceFilesFromPath(const FString& InFullSequencePath, bool bInShouldSort)
{
	TArray<FString> ImageFiles;

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	IFileManager& FileManager = IFileManager::Get();

	bool bIterateResult = 
		FileManager.IterateDirectory(*InFullSequencePath, [&ImageFiles, &ImageWrapperModule](const TCHAR* InFilenameOrDirectory, bool bInIsDirectory)
									 {
										 if (!bInIsDirectory)
										 {
											 EImageFormat Format = ImageWrapperModule.GetImageFormatFromExtension(*FPaths::GetExtension(InFilenameOrDirectory));
											 if (Format != EImageFormat::Invalid)
											 {
												 ImageFiles.Add(InFilenameOrDirectory);
											 }
										 }

										 return true;
									 });

	if (bInShouldSort)
	{
		FRegexPattern Pattern = GetRegexPattern();

		ImageFiles.Sort([&Pattern](const FString& InLeft, const FString& InRight)
						{
							FString Dummy;
							FString LeftDigits, RightDigits;
							bool bSuccess = ExtractInfoFromFileName(Pattern, InLeft, Dummy, LeftDigits, Dummy);
							bSuccess = bSuccess && ExtractInfoFromFileName(Pattern, InRight, Dummy, RightDigits, Dummy);

							if (bSuccess)
							{
								return FCString::Atoi(*LeftDigits) < FCString::Atoi(*RightDigits);
							}

							return InLeft < InRight;
						});
	}

	return ImageFiles;
}

bool GetImageDimensionsFromPath(const FString& InImagePath, FIntPoint& OutDimensions)
{
	TArray<uint8> RawFileData;
	if (FFileHelper::LoadFileToArray(RawFileData, *InImagePath))
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
		EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(RawFileData.GetData(), RawFileData.Num());
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);

		if (ImageWrapper.IsValid() && ImageWrapper->SetCompressed(RawFileData.GetData(), RawFileData.Num()))
		{
			OutDimensions.X = ImageWrapper->GetWidth();
			OutDimensions.Y = ImageWrapper->GetHeight();
			return true;
		}
	}

	return false;
}

}
