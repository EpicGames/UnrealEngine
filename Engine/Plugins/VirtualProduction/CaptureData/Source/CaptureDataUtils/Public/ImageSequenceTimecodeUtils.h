// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timecode.h"
#include "ImgMediaSource.h"

#include "ImageSequenceTimecodeUtils.generated.h"

#define UE_API CAPTUREDATAUTILS_API

UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UImageSequenceTimecodeUtils
	: public UObject
{
	GENERATED_BODY()

public:

	static UE_API const FName TimecodeTagName;
	static UE_API const FName TimecodeRateTagName;

	UFUNCTION(BlueprintCallable, Category = "ImageSequence")
	static UE_API void SetTimecodeInfo(const FTimecode& InTimecode, const FFrameRate& InFrameRate, UImgMediaSource* InImageSequence);

	UFUNCTION(BlueprintCallable, Category = "ImageSequence")
	static UE_API void SetTimecodeInfoString(const FString& InTimecode, const FString& InFrameRate, UImgMediaSource* InImageSequence);

	UFUNCTION(BlueprintCallable, Category = "ImageSequence")
	static UE_API FTimecode GetTimecode(UImgMediaSource* InImageSequence);

	UFUNCTION(BlueprintCallable, Category = "ImageSequence")
	static UE_API FFrameRate GetFrameRate(UImgMediaSource* InImageSequence);

	UFUNCTION(BlueprintCallable, Category = "ImageSequence")
	static UE_API FString GetTimecodeString(UImgMediaSource* InImageSequence);

	UFUNCTION(BlueprintCallable, Category = "ImageSequence")
	static UE_API FString GetFrameRateString(UImgMediaSource* InImageSequence);

	static UE_API bool IsValidTimecodeInfo(const FTimecode& InTimecode, const FFrameRate& InTimecodeRate);
	static UE_API bool IsValidTimecode(const FTimecode& InTimecode);
	static UE_API bool IsValidFrameRate(const FFrameRate& InTimecodeRate);
};

#undef UE_API
