// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timecode.h"
#include "Misc/FrameRate.h"

#include "Containers/UnrealString.h"

CAPTUREDATAUTILS_API FTimecode ParseTimecode(const FString& InTimecodeString);
CAPTUREDATAUTILS_API FFrameRate ConvertFrameRate(double InFrameRate);