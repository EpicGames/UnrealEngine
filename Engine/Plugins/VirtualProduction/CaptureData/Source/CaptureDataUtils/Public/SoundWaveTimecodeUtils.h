// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Misc/Timecode.h"
#include "Misc/FrameRate.h"
#include "Sound/SoundWave.h"

#include "SoundWaveTimecodeUtils.generated.h"

#define UE_API CAPTUREDATAUTILS_API

UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class USoundWaveTimecodeUtils : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Audio")
	static UE_API void SetTimecodeInfo(const FTimecode& InTimecode, const FFrameRate& InFrameRate, USoundWave* OutSoundWave);

	UFUNCTION(BlueprintCallable, Category = "Audio")
	static UE_API FTimecode GetTimecode(const USoundWave* InSoundWave);

	UFUNCTION(BlueprintCallable, Category = "Audio")
	static UE_API FFrameRate GetFrameRate(const USoundWave* InSoundWave);

	UFUNCTION(BlueprintCallable, Category = "Audio")
	static UE_API bool HasTimecode(const USoundWave* InSoundWave);

	UFUNCTION(BlueprintCallable, Category = "Audio")
	static UE_API bool HasTimecodeRate(const USoundWave* InSoundWave);
};

#undef UE_API
