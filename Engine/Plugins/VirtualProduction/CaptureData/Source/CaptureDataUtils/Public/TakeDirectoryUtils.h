// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Internationalization/Regex.h"

#define UE_API CAPTUREDATAUTILS_API

namespace UE::CaptureData
{

UE_API FRegexPattern GetRegexPattern();
UE_API bool ExtractInfoFromFileName(const FRegexPattern& InPattern, const FString& InFileName, FString& OutPrefix, FString& OutDigits, FString& OutExtension);

UE_API FString GetFileFormat(const FRegexPattern& InPattern, const FString& InFileName);
UE_API FString GetFileNameFormat(const FString& InDirectory);

UE_API TArray<FString> GetImageSequenceFilesFromPath(const FString& InFullSequencePath, bool bInShouldSort = true);
UE_API bool GetImageDimensionsFromPath(const FString& InImagePath, FIntPoint& OutDimensions);

}

#undef UE_API